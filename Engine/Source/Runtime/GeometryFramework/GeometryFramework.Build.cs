// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GeometryFramework : ModuleRules
{
	public GeometryFramework(ReadOnlyTargetRules Target) : base(Target)
	{
		//PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"Shaders" // this is required for raytracing suppport?
			}
		);

		// These are required to register filters to the Level Editor Outliner
		if (Target.bCompileAgainstEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"EditorWidgets",
					"Slate"
				}
			);

			PrivateIncludePathModuleNames.AddRange(
				new string[]
				{
					"LevelEditor",
				}
			);
		}

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"RenderCore",
				"RHI",
				"PhysicsCore",
				"InteractiveToolsFramework",
				"MeshDescription",
				"StaticMeshDescription",
				"SkeletalMeshDescription",
				"GeometryCore",
				"MeshConversion"
			}
		);
	}
}
