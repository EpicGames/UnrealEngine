// Copyright Epic Games, Inc. All Rights Reserved.

#include "Changes/MeshChange.h"
#include "DynamicMesh/DynamicMesh3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshChange)

using namespace UE::Geometry;

FMeshChange::FMeshChange() = default;

FMeshChange::FMeshChange(TUniquePtr<FDynamicMeshChange> DynamicMeshChangeIn)
{
	DynamicMeshChange = MoveTemp(DynamicMeshChangeIn);
}

FMeshChange::~FMeshChange() = default;

void FMeshChange::Apply(UObject* Object)
{
	IMeshCommandChangeTarget* ChangeTarget = CastChecked<IMeshCommandChangeTarget>(Object);
	ChangeTarget->ApplyChange(this, false);

	if (OnChangeAppliedFunc)
	{
		OnChangeAppliedFunc(this, Object, true);
	}
}

void FMeshChange::Revert(UObject* Object)
{
	IMeshCommandChangeTarget* ChangeTarget = CastChecked<IMeshCommandChangeTarget>(Object);
	ChangeTarget->ApplyChange(this, true);

	if (OnChangeAppliedFunc)
	{
		OnChangeAppliedFunc(this, Object, false);
	}
}

void FMeshChange::ProcessChangeVertices(const FDynamicMesh3* ChangedMesh, TFunctionRef<void(TConstArrayView<int32>)> ProcessFn, bool bRevert) const
{
	TArray<int32> VerticesOut;
	if (ensure(DynamicMeshChange))
	{
		DynamicMeshChange->GetAffectedVertices(VerticesOut, bRevert);
	}
	ProcessFn(VerticesOut);
}

FString FMeshChange::ToString() const
{
	return FString("Mesh Change");
}


void FMeshChange::ApplyChangeToMesh(FDynamicMesh3* Mesh, bool bRevert) const
{
	DynamicMeshChange->Apply(Mesh, bRevert);
}



