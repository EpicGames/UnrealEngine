// Copyright Epic Games, Inc. All Rights Reserved.

#include "Changes/MeshReplacementChange.h"
#include "DynamicMesh/DynamicMesh3.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshReplacementChange)

using namespace UE::Geometry;

FMeshReplacementChange::FMeshReplacementChange()
{
}

FMeshReplacementChange::FMeshReplacementChange(TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> BeforeIn, TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> AfterIn)
{
	Before = BeforeIn;
	After = AfterIn;
}

void FMeshReplacementChange::Apply(UObject* Object)
{
	IMeshReplacementCommandChangeTarget* ChangeTarget = CastChecked<IMeshReplacementCommandChangeTarget>(Object);
	ChangeTarget->ApplyChange(this, false);

	if (OnChangeAppliedFunc)
	{
		OnChangeAppliedFunc(this, Object, true);
	}
}

void FMeshReplacementChange::Revert(UObject* Object)
{
	IMeshReplacementCommandChangeTarget* ChangeTarget = CastChecked<IMeshReplacementCommandChangeTarget>(Object);
	ChangeTarget->ApplyChange(this, true);

	if (OnChangeAppliedFunc)
	{
		OnChangeAppliedFunc(this, Object, false);
	}
}

void FMeshReplacementChange::ProcessChangeVertices(const FDynamicMesh3* ChangedMesh, TFunctionRef<void(TConstArrayView<int32>)> ProcessFn, bool bRevert) const
{
	check(ChangedMesh);
	TArray<int32> Vertices;
	Vertices.Reserve(ChangedMesh->VertexCount());
	for (int32 VID : ChangedMesh->VertexIndicesItr())
	{
		Vertices.Add(VID);
	}
	ProcessFn(Vertices);
}

FString FMeshReplacementChange::ToString() const
{
	return FString("Mesh Change");
}


