// Copyright Epic Games, Inc. All Rights Reserved.

#include "Changes/MeshVertexChange.h"
#include "DynamicMesh/DynamicMesh3.h"

#include "Components/BaseDynamicMeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshVertexChange)

using namespace UE::Geometry;

void FMeshVertexChange::Apply(UObject* Object)
{
	IMeshVertexCommandChangeTarget* ChangeTarget = CastChecked<IMeshVertexCommandChangeTarget>(Object);
	ChangeTarget->ApplyChange(this, false);
}

void FMeshVertexChange::Revert(UObject* Object)
{
	IMeshVertexCommandChangeTarget* ChangeTarget = CastChecked<IMeshVertexCommandChangeTarget>(Object);
	ChangeTarget->ApplyChange(this, true);
}

void FMeshVertexChange::ProcessChangeVertices(const FDynamicMesh3* ChangedMesh, TFunctionRef<void(TConstArrayView<int32>)> ProcessFn, bool bRevert) const
{
	// Note: Since the FMeshVertexChange doesn't change vertex IDs, we don't need to look at ChangedMesh or bRevert and can just pass the Vertices array
	ProcessFn(Vertices);
}

FString FMeshVertexChange::ToString() const
{
	return FString("Mesh Vertex Change");
}


FMeshVertexChangeBuilder::FMeshVertexChangeBuilder()
{
	Change = MakeUnique<FMeshVertexChange>();
}


FMeshVertexChangeBuilder::FMeshVertexChangeBuilder(EMeshVertexChangeComponents Components)
{
	Change = MakeUnique<FMeshVertexChange>();

	bSavePositions = ((Components & EMeshVertexChangeComponents::VertexPositions) != EMeshVertexChangeComponents::None);
	Change->bHaveVertexPositions = bSavePositions;

	bSaveColors = ((Components & EMeshVertexChangeComponents::VertexColors) != EMeshVertexChangeComponents::None);
	Change->bHaveVertexColors = bSaveColors;
	
	bSaveOverlayNormals = ((Components & EMeshVertexChangeComponents::OverlayNormals) != EMeshVertexChangeComponents::None);
	Change->bHaveOverlayNormals = bSaveOverlayNormals;

	bSaveOverlayUVs = ((Components & EMeshVertexChangeComponents::OverylayUVs) != EMeshVertexChangeComponents::None);
	Change->bHaveOverlayUVs = bSaveOverlayUVs;
}

void FMeshVertexChangeBuilder::UpdateVertex(int VertexID, const FVector3d& OldPosition, const FVector3d& NewPosition)
{
	const int* FoundIndex = SavedVertices.Find(VertexID);
	if (FoundIndex == nullptr)
	{
		int NewIndex = Change->Vertices.Num();
		SavedVertices.Add(VertexID, NewIndex);
		Change->Vertices.Add(VertexID);
		Change->OldPositions.Add(OldPosition);
		Change->NewPositions.Add(NewPosition);
	} 
	else
	{
		Change->NewPositions[*FoundIndex] = NewPosition;
	}
}


void FMeshVertexChangeBuilder::UpdateVertexColor(int32 VertexID, const FVector3f& OldColor, const FVector3f& NewColor)
{
	const int* FoundIndex = SavedVertices.Find(VertexID);
	if (FoundIndex == nullptr)
	{
		int NewIndex = Change->Vertices.Num();
		SavedVertices.Add(VertexID, NewIndex);
		Change->Vertices.Add(VertexID);
		Change->OldColors.Add(OldColor);
		Change->NewColors.Add(NewColor);
	}
	else
	{
		Change->NewColors[*FoundIndex] = NewColor;
	}
}


void FMeshVertexChangeBuilder::UpdateVertexFinal(int VertexID, const FVector3d& NewPosition)
{
	check(SavedVertices.Contains(VertexID));

	const int* Index = SavedVertices.Find(VertexID);
	if ( Index != nullptr )
	{
		Change->NewPositions[*Index] = NewPosition;
	}
}





void FMeshVertexChangeBuilder::SaveVertexInitial(const FDynamicMesh3* Mesh, int32 VertexID)
{
	const int32* FoundIndex = SavedVertices.Find(VertexID);
	if (FoundIndex == nullptr)
	{
		int32 Index = Change->Vertices.Num();
		SavedVertices.Add(VertexID, Index);
		Change->Vertices.Add(VertexID);
		if (bSavePositions)
		{
			FVector3d Pos = Mesh->GetVertex(VertexID);
			Change->OldPositions.Add(Pos);
			Change->NewPositions.Add(Pos);
		}
		if (bSaveColors)
		{
			FVector3f Color = Mesh->GetVertexColor(VertexID);
			Change->OldColors.Add(Color);
			Change->NewColors.Add(Color);
		}
		if (OnNewVertexSaved)
		{
			OnNewVertexSaved(VertexID, Index);
		}
	}
	else
	{
		int32 Index = *FoundIndex;
		if (bSavePositions)
		{
			Change->NewPositions[Index] = Mesh->GetVertex(VertexID);
		}
		if (bSaveColors)
		{
			Change->NewColors[Index] = Mesh->GetVertexColor(VertexID);
		}
	}
}


void FMeshVertexChangeBuilder::SaveVertexFinal(const FDynamicMesh3* Mesh, int32 VertexID)
{
	const int32* FoundIndex = SavedVertices.Find(VertexID);
	if (FoundIndex != nullptr)
	{
		int32 Index = *FoundIndex;
		if (bSavePositions)
		{
			Change->NewPositions[Index] = Mesh->GetVertex(VertexID);
		}
		if (bSaveColors)
		{
			Change->NewColors[Index] = Mesh->GetVertexColor(VertexID);
		}
	}
}












void FMeshVertexChangeBuilder::UpdateOverlayNormal(int ElementID, const FVector3f& OldNormal, const FVector3f& NewNormal)
{
	const int* FoundIndex = SavedNormalElements.Find(ElementID);
	if (FoundIndex == nullptr)
	{
		int NewIndex = Change->Normals.Num();
		SavedNormalElements.Add(ElementID, NewIndex);
		Change->Normals.Add(ElementID);
		Change->OldNormals.Add(OldNormal);
		Change->NewNormals.Add(NewNormal);
	}
	else
	{
		Change->NewNormals[*FoundIndex] = NewNormal;
	}
}

void FMeshVertexChangeBuilder::UpdateOverlayNormalFinal(int ElementID, const FVector3f& NewNormal)
{
	check(SavedNormalElements.Contains(ElementID));
	const int* Index = SavedNormalElements.Find(ElementID);
	if (Index != nullptr)
	{
		Change->NewNormals[*Index] = NewNormal;
	}
}



void FMeshVertexChangeBuilder::SaveOverlayNormals(const FDynamicMesh3* Mesh, const TArray<int>& ElementIDs, bool bInitial)
{
	if (Mesh->HasAttributes() == false || Mesh->Attributes()->PrimaryNormals() == nullptr)
	{
		return;
	}
	const FDynamicMeshNormalOverlay* Overlay = Mesh->Attributes()->PrimaryNormals();

	int Num = ElementIDs.Num();
	if (bInitial)
	{
		for (int k = 0; k < Num; ++k)
		{
			FVector3f Normal = Overlay->GetElement(ElementIDs[k]);
			UpdateOverlayNormal(ElementIDs[k], Normal, Normal);
		}
	}
	else
	{
		for (int k = 0; k < Num; ++k)
		{
			FVector3f Normal = Overlay->GetElement(ElementIDs[k]);
			UpdateOverlayNormalFinal(ElementIDs[k], Normal);
		}
	}
}


void FMeshVertexChangeBuilder::SaveOverlayNormals(const FDynamicMesh3* Mesh, const TSet<int>& ElementIDs, bool bInitial)
{
	if (Mesh->HasAttributes() == false || Mesh->Attributes()->PrimaryNormals() == nullptr)
	{
		return;
	}
	const FDynamicMeshNormalOverlay* Overlay = Mesh->Attributes()->PrimaryNormals();

	if (bInitial)
	{
		for (int ElementID : ElementIDs)
		{
			FVector3f Normal = Overlay->GetElement(ElementID);
			UpdateOverlayNormal(ElementID, Normal, Normal);
		}
	}
	else
	{
		for (int ElementID : ElementIDs)
		{
			FVector3f Normal = Overlay->GetElement(ElementID);
			UpdateOverlayNormalFinal(ElementID, Normal);
		}
	}
}

void FMeshVertexChangeBuilder::UpdateOverlayUV(int ElementID, const FVector2f& OldUV, const FVector2f& NewUV)
{
	const int* FoundIndex = SavedUVElements.Find(ElementID);
	if (FoundIndex == nullptr)
	{
		int NewIndex = Change->UVs.Num();
		SavedUVElements.Add(ElementID, NewIndex);
		Change->UVs.Add(ElementID);
		Change->OldUVs.Add(OldUV);
		Change->NewUVs.Add(NewUV);
	}
	else
	{
		Change->NewUVs[*FoundIndex] = NewUV;
	}
}

void FMeshVertexChangeBuilder::UpdateOverlayUVFinal(int ElementID, const FVector2f& NewUV)
{
	check(SavedUVElements.Contains(ElementID));
	const int* Index = SavedUVElements.Find(ElementID);
	if (Index != nullptr)
	{
		Change->NewUVs[*Index] = NewUV;
	}
}

void FMeshVertexChangeBuilder::SaveOverlayUVs(const FDynamicMesh3* Mesh, const TArray<int>& ElementIDs, bool bInitial)
{
	if (Mesh->HasAttributes() == false || Mesh->Attributes()->PrimaryUV() == nullptr)
	{
		return;
	}
	const FDynamicMeshUVOverlay* Overlay = Mesh->Attributes()->PrimaryUV();

	int Num = ElementIDs.Num();
	if (bInitial)
	{
		for (int k = 0; k < Num; ++k)
		{
			FVector2f UV = Overlay->GetElement(ElementIDs[k]);
			UpdateOverlayUV(ElementIDs[k], UV, UV);
		}
	}
	else
	{
		for (int k = 0; k < Num; ++k)
		{
			FVector2f UV = Overlay->GetElement(ElementIDs[k]);
			UpdateOverlayUVFinal(ElementIDs[k], UV);
		}
	}
}

void FMeshVertexChangeBuilder::SaveOverlayUVs(const FDynamicMesh3* Mesh, const TSet<int>& ElementIDs, bool bInitial)
{
	if (Mesh->HasAttributes() == false || Mesh->Attributes()->PrimaryUV() == nullptr)
	{
		return;
	}
	const FDynamicMeshUVOverlay* Overlay = Mesh->Attributes()->PrimaryUV();

	if (bInitial)
	{
		for (int ElementID : ElementIDs)
		{
			FVector2f UV = Overlay->GetElement(ElementID);
			UpdateOverlayUV(ElementID, UV, UV);
		}
	}
	else
	{
		for (int ElementID : ElementIDs)
		{
			FVector2f UV = Overlay->GetElement(ElementID);
			UpdateOverlayUVFinal(ElementID, UV);
		}
	}
}