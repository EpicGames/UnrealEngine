// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Components/BaseDynamicMeshComponent.h"
#include "Components/BaseDynamicMeshSceneProxy.h"
#include "Engine/Engine.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BaseDynamicMeshComponent)

using namespace UE::Geometry;

UBaseDynamicMeshComponent::UBaseDynamicMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}



#if WITH_EDITOR
void UBaseDynamicMeshComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropName = PropertyChangedEvent.GetPropertyName();
	if ( (PropName == GET_MEMBER_NAME_CHECKED(UBaseDynamicMeshComponent, bEnableRaytracing)) || 
		 (PropName == GET_MEMBER_NAME_CHECKED(UBaseDynamicMeshComponent, DrawPath)) )
	{
		OnRenderingStateChanged(true);
	}
	else if ( (PropName == GET_MEMBER_NAME_CHECKED(UBaseDynamicMeshComponent, bEnableViewModeOverrides))  )
	{
		OnRenderingStateChanged(false);
	}
}
#endif


void UBaseDynamicMeshComponent::SetShadowsEnabled(bool bEnabled)
{
	FlushRenderingCommands();
	SetCastShadow(bEnabled);
	OnRenderingStateChanged(true);
}


void UBaseDynamicMeshComponent::SetViewModeOverridesEnabled(bool bEnabled)
{
	if (bEnableViewModeOverrides != bEnabled)
	{
		bEnableViewModeOverrides = bEnabled;
		OnRenderingStateChanged(false);
	}
}


void UBaseDynamicMeshComponent::SetOverrideRenderMaterial(UMaterialInterface* Material)
{
	if (OverrideRenderMaterial != Material)
	{
		OverrideRenderMaterial = Material;
		NotifyMaterialSetUpdated();
	}
}

void UBaseDynamicMeshComponent::ClearOverrideRenderMaterial()
{
	if (OverrideRenderMaterial != nullptr)
	{
		OverrideRenderMaterial = nullptr;
		NotifyMaterialSetUpdated();
	}
}





void UBaseDynamicMeshComponent::SetSecondaryRenderMaterial(UMaterialInterface* Material)
{
	if (SecondaryRenderMaterial != Material)
	{
		SecondaryRenderMaterial = Material;
		NotifyMaterialSetUpdated();
	}
}

void UBaseDynamicMeshComponent::ClearSecondaryRenderMaterial()
{
	if (SecondaryRenderMaterial != nullptr)
	{
		SecondaryRenderMaterial = nullptr;
		NotifyMaterialSetUpdated();
	}
}

void UBaseDynamicMeshComponent::SetOverrideWireframeRenderMaterial(UMaterialInterface* Material)
{
	if (WireframeMaterialOverride != Material)
	{
		WireframeMaterialOverride = Material;
		NotifyMaterialSetUpdated();
	}
}

void UBaseDynamicMeshComponent::ClearOverrideWireframeRenderMaterial()
{
	if (WireframeMaterialOverride != nullptr)
	{
		WireframeMaterialOverride = nullptr;
		NotifyMaterialSetUpdated();
	}
}

void UBaseDynamicMeshComponent::SetOverrideSecondaryWireframeRenderMaterial(UMaterialInterface* Material)
{
	if (SecondaryWireframeMaterialOverride != Material)
	{
		SecondaryWireframeMaterialOverride = Material;
		NotifyMaterialSetUpdated();
	}
}

void UBaseDynamicMeshComponent::ClearOverrideSecondaryWireframeRenderMaterial()
{
	if (SecondaryWireframeMaterialOverride != nullptr)
	{
		SecondaryWireframeMaterialOverride = nullptr;
		NotifyMaterialSetUpdated();
	}
}

void UBaseDynamicMeshComponent::SetSecondaryBuffersVisibility(bool bSecondaryVisibility)
{
	bDrawSecondaryBuffers = bSecondaryVisibility;
}

bool UBaseDynamicMeshComponent::GetSecondaryBuffersVisibility() const
{
	return bDrawSecondaryBuffers;
}


void UBaseDynamicMeshComponent::SetEnableRaytracing(bool bSetEnabled)
{
	if (bEnableRaytracing != bSetEnabled)
	{
		bEnableRaytracing = bSetEnabled;
		OnRenderingStateChanged(true);
	}
}

bool UBaseDynamicMeshComponent::GetEnableRaytracing() const
{
	return bEnableRaytracing;
}




void UBaseDynamicMeshComponent::SetMeshDrawPath(EDynamicMeshDrawPath NewDrawPath)
{
	if (DrawPath != NewDrawPath)
	{
		DrawPath = NewDrawPath;
		OnRenderingStateChanged(true);
	}
}

EDynamicMeshDrawPath UBaseDynamicMeshComponent::GetMeshDrawPath() const
{
	return DrawPath;
}


void UBaseDynamicMeshComponent::SetColorOverrideMode(EDynamicMeshComponentColorOverrideMode NewMode)
{
	if (ColorMode != NewMode)
	{
		ColorMode = NewMode;
		OnRenderingStateChanged(false);
	}
}


void UBaseDynamicMeshComponent::SetConstantOverrideColor(FColor NewColor) 
{ 
	if (ConstantColor != NewColor)
	{
		ConstantColor = NewColor; 
		OnRenderingStateChanged(false);
	}
}


void UBaseDynamicMeshComponent::SetVertexColorSpaceTransformMode(EDynamicMeshVertexColorTransformMode NewMode)
{
	if (ColorSpaceMode != NewMode)
	{
		ColorSpaceMode = NewMode;
		OnRenderingStateChanged(false);
	}
}

void UBaseDynamicMeshComponent::SetTwoSided(bool bEnable)
{
	if (bTwoSided != bEnable)
	{
		bTwoSided = bEnable;
		OnRenderingStateChanged(false);
	}
}

void UBaseDynamicMeshComponent::SetEnableFlatShading(bool bEnable)
{
	if (bEnableFlatShading != bEnable)
	{
		bEnableFlatShading = bEnable; 
		OnRenderingStateChanged(false);
	}
}


void UBaseDynamicMeshComponent::OnRenderingStateChanged(bool bForceImmedateRebuild)
{
	if (bForceImmedateRebuild)
	{
		if (GetSceneProxy())
		{
			// finish any drawing so that we can be certain our SceneProxy is no longer in use before we rebuild it below
			FlushRenderingCommands();
		}

		// force immediate rebuild of the SceneProxy
		if (IsRegistered())
		{
			ReregisterComponent();
		}
	}
	else
	{
		MarkRenderStateDirty();
	}
}


int32 UBaseDynamicMeshComponent::GetNumMaterials() const
{
	return BaseMaterials.Num();
}

UMaterialInterface* UBaseDynamicMeshComponent::GetMaterial(int32 ElementIndex) const 
{
	return (ElementIndex >= 0 && ElementIndex < BaseMaterials.Num()) ? BaseMaterials[ElementIndex] : nullptr;
}

// Deprecated in 5.7
FMaterialRelevance UBaseDynamicMeshComponent::GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const
{
	return GetMaterialRelevance(GetFeatureLevelShaderPlatform_Checked(InFeatureLevel));
}

FMaterialRelevance UBaseDynamicMeshComponent::GetMaterialRelevance(EShaderPlatform InShaderPlatform) const
{
	FMaterialRelevance Result = UMeshComponent::GetMaterialRelevance(InShaderPlatform);
	if (OverrideRenderMaterial)
	{
		Result |= OverrideRenderMaterial->GetRelevance_Concurrent(InShaderPlatform);
	}
	if (SecondaryRenderMaterial)
	{
		Result |= SecondaryRenderMaterial->GetRelevance_Concurrent(InShaderPlatform);
	}
	return Result;
}

// Note Dynamic Meshes don't really have named material slots, so we generate slot names from material + index
namespace UE::Private::DynamicMeshMaterialSlotNameHelper
{
	static FName GetMaterialSlotName(const UMaterialInterface* Mat, int32 MaterialIndex)
	{
		return FName(FString::Printf(TEXT("%s_%d"), *((Mat) ? Mat->GetName() : TEXT("Material")), MaterialIndex));
	}
}
TArray<FName> UBaseDynamicMeshComponent::GetMaterialSlotNames() const
{
	using namespace UE::Private::DynamicMeshMaterialSlotNameHelper;
	TArray<FName> ToRet;
	const int32 NumMaterials = GetNumMaterials();
	ToRet.Reserve(NumMaterials);
	for (int32 Idx = 0; Idx < NumMaterials; ++Idx)
	{
		ToRet.Add(GetMaterialSlotName(GetMaterial(Idx), Idx));
	}
	return ToRet;
}

bool UBaseDynamicMeshComponent::IsMaterialSlotNameValid(FName MaterialSlotName) const
{
	using namespace UE::Private::DynamicMeshMaterialSlotNameHelper;
	const int32 NumMaterials = GetNumMaterials();
	for (int32 Idx = 0; Idx < NumMaterials; ++Idx)
	{
		if (MaterialSlotName == GetMaterialSlotName(GetMaterial(Idx), Idx))
		{
			return true;
		}
	}
	return false;
}

UMaterialInterface* UBaseDynamicMeshComponent::GetMaterialByName(FName MaterialSlotName) const
{
	using namespace UE::Private::DynamicMeshMaterialSlotNameHelper;
	const int32 NumMaterials = GetNumMaterials();
	for (int32 Idx = 0; Idx < BaseMaterials.Num(); ++Idx)
	{
		if (MaterialSlotName == GetMaterialSlotName(GetMaterial(Idx), Idx))
		{
			return GetMaterial(Idx);
		}
	}
	return nullptr;
}

void UBaseDynamicMeshComponent::SetMaterial(int32 ElementIndex, UMaterialInterface* Material)
{
	check(ElementIndex >= 0);
	if (ElementIndex >= BaseMaterials.Num())
	{
		BaseMaterials.SetNum(ElementIndex + 1, EAllowShrinking::No);
	}
	BaseMaterials[ElementIndex] = Material;

	// @todo allow for precache of pipeline state objects for rendering
	// PrecachePSOs(); // indirectly calls GetUsedMaterials, requires CollectPSOPrecacheData to be implemented, see UStaticMeshComponent for example

	
	MarkRenderStateDirty();

	// update the body instance in case this material has an associated physics material 
	FBodyInstance* BodyInst = GetBodyInstance();
	if (BodyInst && BodyInst->IsValidBodyInstance())
	{
		BodyInst->UpdatePhysicalMaterials();
	}
}


void UBaseDynamicMeshComponent::SetNumMaterials(int32 NumMaterials)
{
	if (BaseMaterials.Num() > NumMaterials)
	{
		// discard extra materials
		BaseMaterials.SetNum(NumMaterials);
	}
	else
	{
		while (NumMaterials < BaseMaterials.Num())
		{
			SetMaterial(NumMaterials, nullptr);
			NumMaterials++;
		}
	}
}

void UBaseDynamicMeshComponent::GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials) const
{
	UMeshComponent::GetUsedMaterials(OutMaterials, bGetDebugMaterials);
	if (OverrideRenderMaterial != nullptr)
	{
		OutMaterials.Add(OverrideRenderMaterial);
	}
	if (SecondaryRenderMaterial != nullptr)
	{
		OutMaterials.Add(SecondaryRenderMaterial);
	}
	if (ColorMode != EDynamicMeshComponentColorOverrideMode::None && UBaseDynamicMeshComponent::DefaultVertexColorMaterial != nullptr)
	{
		OutMaterials.Add(UBaseDynamicMeshComponent::DefaultVertexColorMaterial);
	}
}


UMaterialInterface* UBaseDynamicMeshComponent::DefaultWireframeMaterial = nullptr;
UMaterialInterface* UBaseDynamicMeshComponent::DefaultVertexColorMaterial = nullptr;

void UBaseDynamicMeshComponent::InitializeDefaultMaterials()
{
	if (GEngine->WireframeMaterial != nullptr)
	{
		DefaultWireframeMaterial = GEngine->WireframeMaterial;
	}
	if (GEngine->VertexColorViewModeMaterial_ColorOnly != nullptr)
	{
		DefaultVertexColorMaterial = GEngine->VertexColorViewModeMaterial_ColorOnly;
	}
}

UMaterialInterface* UBaseDynamicMeshComponent::GetDefaultWireframeMaterial_RenderThread()
{
	return DefaultWireframeMaterial;
}

void UBaseDynamicMeshComponent::SetDefaultWireframeMaterial(UMaterialInterface* Material)
{
	ENQUEUE_RENDER_COMMAND(BaseDynamicMeshComponent_SetWireframeMaterial)(
		[Material](FRHICommandListImmediate& RHICmdList)
	{
		UBaseDynamicMeshComponent::DefaultWireframeMaterial = Material;
	});
}

UMaterialInterface* UBaseDynamicMeshComponent::GetDefaultVertexColorMaterial_RenderThread()
{
	return DefaultVertexColorMaterial;
}

void UBaseDynamicMeshComponent::SetDefaultVertexColorMaterial(UMaterialInterface* Material)
{
	ENQUEUE_RENDER_COMMAND(BaseDynamicMeshComponent_SetVertexColorMaterial)(
		[Material](FRHICommandListImmediate& RHICmdList)
	{
		UBaseDynamicMeshComponent::DefaultVertexColorMaterial = Material;
	});
}