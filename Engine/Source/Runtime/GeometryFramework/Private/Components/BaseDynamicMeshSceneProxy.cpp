// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Components/BaseDynamicMeshSceneProxy.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"
#include "Materials/MaterialRenderProxy.h"
#include "PhysicsEngine/BodySetup.h"
#include "RayTracingDefinitions.h"
#include "RayTracingInstance.h"
#include "SceneInterface.h"
#include "PrimitiveUniformShaderParametersBuilder.h"
#include "PrimitiveDrawingUtils.h"
#include "Engine/Engine.h"		// for GEngine definition
#include "MeshCardRepresentation.h"
#include "MeshCardBuild.h"
#include "DistanceFieldAtlas.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "MeshPaintVisualize.h"

#include "Implicit/SweepingMeshSDF.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "SceneView.h"
#include "Spatial/FastWinding.h"
#include "HAL/IConsoleManager.h"

static TAutoConsoleVariable<bool> CVarDynamicMeshComponent_AllowMeshCardGeneration(
	TEXT("geometry.DynamicMesh.AllowMeshCardGeneration"),
	1,
	TEXT("Whether to allow mesh card generation for dynamic mesh components")
);


namespace UE::DynamicMesh
{

	static bool AllowLumenCardGeneration()
	{
		return CVarDynamicMeshComponent_AllowMeshCardGeneration.GetValueOnAnyThread() && FDataDrivenShaderPlatformInfo::GetSupportsLumenGI(GetFeatureLevelShaderPlatform(GMaxRHIFeatureLevel));
	}
}

FBaseDynamicMeshSceneProxy::FBaseDynamicMeshSceneProxy(UBaseDynamicMeshComponent* Component)
	: FPrimitiveSceneProxy(Component),
	ParentBaseComponent(Component),
	bTwoSided(Component->GetTwoSided()),
	bEnableRaytracing(Component->GetEnableRaytracing()),
	bEnableViewModeOverrides(Component->GetViewModeOverridesEnabled()),
	bPreferStaticDrawPath(Component->GetMeshDrawPath() == EDynamicMeshDrawPath::StaticDraw)
{
	MeshRenderBufferSetConverter.ColorSpaceTransformMode = Component->GetVertexColorSpaceTransformMode();

	if (Component->GetColorOverrideMode() == EDynamicMeshComponentColorOverrideMode::Constant)
	{
		MeshRenderBufferSetConverter.ConstantVertexColor = Component->GetConstantOverrideColor();
		MeshRenderBufferSetConverter.bIgnoreVertexColors = true;
	}

	MeshRenderBufferSetConverter.bUsePerTriangleNormals = Component->GetFlatShadingEnabled();
	
	SetCollisionData();

	FMaterialRelevance MaterialRelevance = Component->GetMaterialRelevance(GetScene().GetShaderPlatform());
	bOpaqueOrMasked = MaterialRelevance.bOpaque;

	// set distance field flags to false
	bool bWillHaveDistanceField = false;
	bSupportsDistanceFieldRepresentation = bWillHaveDistanceField;
	bAffectDistanceFieldLighting = bWillHaveDistanceField;
	// note whether lumen is enabled will depend on the distance field flags (in some cases)
	UpdateVisibleInLumenScene();

	// Dynamic meshes can write to runtime virtual texture if they are set to do so.
	bSupportsRuntimeVirtualTexture = true;
}

// Note: deprecation warnings disabled due to removal of distance field support
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FBaseDynamicMeshSceneProxy::~FBaseDynamicMeshSceneProxy()
{
	// destroy all existing renderbuffers
	for (FMeshRenderBufferSet* BufferSet : AllocatedBufferSets)
	{
		FMeshRenderBufferSet::DestroyRenderBufferSet(BufferSet);
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FMeshRenderBufferSet* FBaseDynamicMeshSceneProxy::AllocateNewRenderBufferSet()
{
	// should we hang onto these and destroy them in constructor? leaving to subclass seems risky?
	FMeshRenderBufferSet* RenderBufferSet = new FMeshRenderBufferSet(GetScene().GetFeatureLevel());

	RenderBufferSet->Material = UMaterial::GetDefaultMaterial(MD_Surface);
	RenderBufferSet->bEnableRaytracing = this->bEnableRaytracing && this->IsVisibleInRayTracing();

	AllocatedSetsLock.Lock();
	AllocatedBufferSets.Add(RenderBufferSet);
	AllocatedSetsLock.Unlock();

	return RenderBufferSet;
}

void FBaseDynamicMeshSceneProxy::ReleaseRenderBufferSet(FMeshRenderBufferSet* BufferSet)
{
	FScopeLock Lock(&AllocatedSetsLock);
	if (ensure(AllocatedBufferSets.Contains(BufferSet)))
	{
		AllocatedBufferSets.Remove(BufferSet);
		Lock.Unlock();

		FMeshRenderBufferSet::DestroyRenderBufferSet(BufferSet);
	}
}

int32 FBaseDynamicMeshSceneProxy::GetNumMaterials() const
{
	return ParentBaseComponent->GetNumMaterials();
}

UMaterialInterface* FBaseDynamicMeshSceneProxy::GetMaterial(int32 k) const
{
	UMaterialInterface* Material = ParentBaseComponent->GetMaterial(k);
	return (Material != nullptr) ? Material : UMaterial::GetDefaultMaterial(MD_Surface);
}

void FBaseDynamicMeshSceneProxy::UpdatedReferencedMaterials()
{
#if WITH_EDITOR
	TArray<UMaterialInterface*> Materials;
	ParentBaseComponent->GetUsedMaterials(Materials, true);

	// Temporarily disable material verification while the enqueued render command is in flight.
	// The original value for bVerifyUsedMaterials gets restored when the command is executed.
	// If we do not do this, material verification might spuriously fail in cases where the render command for changing
	// the verfifcation material is still in flight but the render thread is already trying to render the mesh.
	const uint8 bRestoreVerifyUsedMaterials = bVerifyUsedMaterials;
	bVerifyUsedMaterials = false;

	ENQUEUE_RENDER_COMMAND(FMeshRenderBufferSetDestroy)(
		[this, Materials, bRestoreVerifyUsedMaterials](FRHICommandListImmediate& RHICmdList)
	{
		this->SetUsedMaterialForVerification(Materials);
		this->bVerifyUsedMaterials = bRestoreVerifyUsedMaterials;
	});
#endif
}

FMaterialRenderProxy* FBaseDynamicMeshSceneProxy::GetEngineVertexColorMaterialProxy(FMeshElementCollector& Collector, const FEngineShowFlags& EngineShowFlags, bool bProxyIsSelected, bool bIsHovered)
{
	FMaterialRenderProxy* ForceOverrideMaterialProxy = nullptr;
#if UE_ENABLE_DEBUG_DRAWING
	if (bProxyIsSelected && EngineShowFlags.VertexColors && AllowDebugViewmodes())
	{
		// Note: static mesh renderer does something more complicated involving per-section selection, but whole component selection seems ok for now.
		if (FMaterialRenderProxy* VertexColorVisualizationMaterialInstance = MeshPaintVisualize::GetMaterialRenderProxy(bProxyIsSelected, bIsHovered))
		{
			Collector.RegisterOneFrameMaterialProxy(VertexColorVisualizationMaterialInstance);
			ForceOverrideMaterialProxy = VertexColorVisualizationMaterialInstance;
		}
	}
#endif
	return ForceOverrideMaterialProxy;
}

bool FBaseDynamicMeshSceneProxy::IsCollisionView(const FEngineShowFlags& EngineShowFlags, bool& bDrawSimpleCollision, bool& bDrawComplexCollision) const
{
	bDrawSimpleCollision = bDrawComplexCollision = false;

	bool bDrawCollisionView = (EngineShowFlags.CollisionVisibility || EngineShowFlags.CollisionPawn);

#if UE_ENABLE_DEBUG_DRAWING
	// If in a 'collision view' and collision is enabled
	FScopeLock Lock(&CachedCollisionLock);
	if (bHasCollisionData && bDrawCollisionView && IsCollisionEnabled())
	{
		// See if we have a response to the interested channel
		bool bHasResponse = EngineShowFlags.CollisionPawn && CollisionResponse.GetResponse(ECC_Pawn) != ECR_Ignore;
		bHasResponse |= EngineShowFlags.CollisionVisibility && CollisionResponse.GetResponse(ECC_Visibility) != ECR_Ignore;

		if(bHasResponse)
		{
			// Visibility uses complex and pawn uses simple. However, if UseSimpleAsComplex or UseComplexAsSimple is used we need to adjust accordingly
			bDrawComplexCollision = (EngineShowFlags.CollisionVisibility && CollisionTraceFlag != ECollisionTraceFlag::CTF_UseSimpleAsComplex) || (EngineShowFlags.CollisionPawn && CollisionTraceFlag == ECollisionTraceFlag::CTF_UseComplexAsSimple);
			bDrawSimpleCollision  = (EngineShowFlags.CollisionPawn && CollisionTraceFlag != ECollisionTraceFlag::CTF_UseComplexAsSimple) || (EngineShowFlags.CollisionVisibility && CollisionTraceFlag == ECollisionTraceFlag::CTF_UseSimpleAsComplex);
		}
	}
#endif
	return bDrawCollisionView;
}

void FBaseDynamicMeshSceneProxy::GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BaseDynamicMeshSceneProxy_GetDynamicMeshElements);

	const FEngineShowFlags& EngineShowFlags = ViewFamily.EngineShowFlags;
	bool bIsWireframeViewMode = (AllowDebugViewmodes() && EngineShowFlags.Wireframe);
	bool bWantWireframeOnShaded = ParentBaseComponent->GetEnableWireframeRenderPass();
	bool bWireframe = bIsWireframeViewMode || bWantWireframeOnShaded;
	const bool bProxyIsSelected = IsSelected();


	TArray<FMeshRenderBufferSet*> Buffers;
	GetActiveRenderBufferSets(Buffers);

#if UE_ENABLE_DEBUG_DRAWING
	bool bDrawSimpleCollision = false, bDrawComplexCollision = false;
	const bool bDrawCollisionView = IsCollisionView(EngineShowFlags, bDrawSimpleCollision, bDrawComplexCollision);

	// If we're in a collision view, run the only draw the collision and return without drawing mesh normally
	if (bDrawCollisionView)
	{
		GetCollisionDynamicMeshElements(Buffers, EngineShowFlags, bDrawCollisionView, bDrawSimpleCollision, bDrawComplexCollision, bProxyIsSelected, Views, VisibilityMap, Collector);
		return;
	}
#endif

	// Get wireframe material proxy if requested and available, otherwise disable wireframe
	FMaterialRenderProxy* WireframeMaterialProxy = nullptr;
	if (bWireframe)
	{
		if(ParentBaseComponent->HasOverrideWireframeRenderMaterial())
		{
			WireframeMaterialProxy = ParentBaseComponent->GetOverrideWireframeRenderMaterial()->GetRenderProxy();
		}
		else
		{
			UMaterialInterface* WireframeMaterial = UBaseDynamicMeshComponent::GetDefaultWireframeMaterial_RenderThread();
			if (WireframeMaterial != nullptr)
			{
				FLinearColor UseWireframeColor = (bProxyIsSelected && (bWantWireframeOnShaded == false || bIsWireframeViewMode))  ?
					GEngine->GetSelectedMaterialColor() : ParentBaseComponent->WireframeColor;
				FColoredMaterialRenderProxy* WireframeMaterialInstance = new FColoredMaterialRenderProxy(
					WireframeMaterial->GetRenderProxy(), UseWireframeColor);
				Collector.RegisterOneFrameMaterialProxy(WireframeMaterialInstance);
				WireframeMaterialProxy = WireframeMaterialInstance;
			}
			else
			{
				bWireframe = false;
			}
		}
	}

	FMaterialRenderProxy* ForceOverrideMaterialProxy = GetEngineVertexColorMaterialProxy(Collector, EngineShowFlags, bProxyIsSelected, IsHovered());
	// If engine show flags aren't setting vertex color, also check if the component requested custom vertex color modes for the dynamic mesh
	if (!ForceOverrideMaterialProxy)
	{
		const bool bVertexColor = ParentBaseComponent->ColorMode == EDynamicMeshComponentColorOverrideMode::VertexColors ||
			ParentBaseComponent->ColorMode == EDynamicMeshComponentColorOverrideMode::Polygroups ||
			ParentBaseComponent->ColorMode == EDynamicMeshComponentColorOverrideMode::Constant;
		if (bVertexColor)
		{
			ForceOverrideMaterialProxy = UBaseDynamicMeshComponent::GetDefaultVertexColorMaterial_RenderThread()->GetRenderProxy();
		}
	}

	ESceneDepthPriorityGroup DepthPriority = SDPG_World;


	FMaterialRenderProxy* SecondaryMaterialProxy = ForceOverrideMaterialProxy;
	if (ParentBaseComponent->HasSecondaryRenderMaterial() && ForceOverrideMaterialProxy == nullptr)
	{
		SecondaryMaterialProxy = ParentBaseComponent->GetSecondaryRenderMaterial()->GetRenderProxy();
	}
	bool bDrawSecondaryBuffers = ParentBaseComponent->GetSecondaryBuffersVisibility();

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];

			// Draw the mesh.
			for (FMeshRenderBufferSet* BufferSet : Buffers)
			{
				FMaterialRenderProxy* MaterialProxy = ForceOverrideMaterialProxy;
				if (!MaterialProxy)
				{
					UMaterialInterface* UseMaterial = BufferSet->Material;
					if (ParentBaseComponent->HasOverrideRenderMaterial(0))
					{
						UseMaterial = ParentBaseComponent->GetOverrideRenderMaterial(0);
					}
					MaterialProxy = UseMaterial->GetRenderProxy();
				}

				if (BufferSet->TriangleCount == 0)
				{
					continue;
				}

				// lock buffers so that they aren't modified while we are submitting them
				FScopeLock BuffersLock(&BufferSet->BuffersLock);

				// do we need separate one of these for each MeshRenderBufferSet?
				FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
				FPrimitiveUniformShaderParametersBuilder Builder;
				BuildUniformShaderParameters(Builder);
				DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), Builder);

				// If we want Wireframe-on-Shaded, we have to draw the solid. If View Mode Overrides are enabled, the solid
				// will be replaced with it's wireframe, so we might as well not. 
				bool bDrawSolidWithWireframe = ( bWantWireframeOnShaded && (bIsWireframeViewMode == false || bEnableViewModeOverrides == false) );

				if (BufferSet->IndexBuffer.Indices.Num() > 0)
				{
					if (bWireframe)
					{
						if (bDrawSolidWithWireframe)
						{
							DrawBatch(Collector, *BufferSet, BufferSet->IndexBuffer, MaterialProxy, /*bWireframe*/false, DepthPriority, ViewIndex, DynamicPrimitiveUniformBuffer);
						}
						DrawBatch(Collector, *BufferSet, BufferSet->IndexBuffer, WireframeMaterialProxy, /*bWireframe*/true, DepthPriority, ViewIndex, DynamicPrimitiveUniformBuffer);
					}
					else
					{
						DrawBatch(Collector, *BufferSet, BufferSet->IndexBuffer, MaterialProxy, /*bWireframe*/false, DepthPriority, ViewIndex, DynamicPrimitiveUniformBuffer);
					}
				}

				// draw secondary buffer if we have it, falling back to base material if we don't have the Secondary material
				FMaterialRenderProxy* UseSecondaryMaterialProxy = (SecondaryMaterialProxy != nullptr) ? SecondaryMaterialProxy : MaterialProxy;
				if (bDrawSecondaryBuffers && BufferSet->SecondaryIndexBuffer.Indices.Num() > 0 && UseSecondaryMaterialProxy != nullptr)
				{
					if (bWireframe)
					{
						if (bDrawSolidWithWireframe)
						{
							DrawBatch(Collector, *BufferSet, BufferSet->SecondaryIndexBuffer, UseSecondaryMaterialProxy, /*bWireframe*/false, DepthPriority, ViewIndex, DynamicPrimitiveUniformBuffer);
						}
						FMaterialRenderProxy* UseSecondaryMaterialProxyWireFrame = ParentBaseComponent->HasOverrideSecondaryWireframeRenderMaterial() ? 
							ParentBaseComponent->GetOverrideSecondaryWireframeRenderMaterial()->GetRenderProxy() : UseSecondaryMaterialProxy;
						DrawBatch(Collector, *BufferSet, BufferSet->SecondaryIndexBuffer, UseSecondaryMaterialProxyWireFrame, /*bWireframe*/true, DepthPriority, ViewIndex, DynamicPrimitiveUniformBuffer);
					}
					else
					{
						DrawBatch(Collector, *BufferSet, BufferSet->SecondaryIndexBuffer, UseSecondaryMaterialProxy, /*bWireframe*/false, DepthPriority, ViewIndex, DynamicPrimitiveUniformBuffer);
					}
				}
			}
		}
	}

#if UE_ENABLE_DEBUG_DRAWING
	GetCollisionDynamicMeshElements(Buffers, EngineShowFlags, bDrawCollisionView, bDrawSimpleCollision, bDrawComplexCollision, bProxyIsSelected, Views, VisibilityMap, Collector);
#endif
}

void FBaseDynamicMeshSceneProxy::GetCollisionDynamicMeshElements(TArray<FMeshRenderBufferSet*>& Buffers, 
	const FEngineShowFlags& EngineShowFlags, bool bDrawCollisionView, bool bDrawSimpleCollision, bool bDrawComplexCollision, bool bProxyIsSelected,
	const TArray<const FSceneView*>& Views, uint32 VisibilityMap, FMeshElementCollector& Collector) const
{
#if UE_ENABLE_DEBUG_DRAWING
	FScopeLock Lock(&CachedCollisionLock);

	if (!bHasCollisionData)
	{
		return;
	}

	// Note: This is closely following StaticMeshSceneProxy.cpp's collision rendering code, from its GetDynamicMeshElements() implementation
	FColor SimpleCollisionColor = FColor(157, 149, 223, 255);
	FColor ComplexCollisionColor = FColor(0, 255, 255, 255);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if (VisibilityMap & (1 << ViewIndex))
		{
			const FSceneView* View = Views[ViewIndex];

			if(AllowDebugViewmodes())
			{
				// Should we draw the mesh wireframe to indicate we are using the mesh as collision
				bool bDrawComplexWireframeCollision = (EngineShowFlags.Collision && IsCollisionEnabled() && CollisionTraceFlag == ECollisionTraceFlag::CTF_UseComplexAsSimple);

				// If drawing complex collision as solid or wireframe
				if (bHasComplexMeshData && (bDrawComplexWireframeCollision || (bDrawCollisionView && bDrawComplexCollision)))
				{
					bool bDrawWireframe = !bDrawCollisionView;

					UMaterial* MaterialToUse = UMaterial::GetDefaultMaterial(MD_Surface);
					FLinearColor DrawCollisionColor = GetWireframeColor();
					// Collision view modes draw collision mesh as solid
					if(bDrawCollisionView)
					{
						MaterialToUse = GEngine->ShadedLevelColorationUnlitMaterial;
					}
					// Wireframe, choose color based on complex or simple
					else
					{
						MaterialToUse = GEngine->WireframeMaterial;
						DrawCollisionColor = (CollisionTraceFlag == ECollisionTraceFlag::CTF_UseComplexAsSimple) ? SimpleCollisionColor : ComplexCollisionColor;
					}
					// Create colored proxy
					FColoredMaterialRenderProxy* CollisionMaterialInstance = new FColoredMaterialRenderProxy(MaterialToUse->GetRenderProxy(), DrawCollisionColor);
					Collector.RegisterOneFrameMaterialProxy(CollisionMaterialInstance);

					// Draw the mesh with collision materials
					for (FMeshRenderBufferSet* BufferSet : Buffers)
					{

						if (BufferSet->TriangleCount == 0)
						{
							continue;
						}

						// lock buffers so that they aren't modified while we are submitting them
						FScopeLock BuffersLock(&BufferSet->BuffersLock);

						// do we need separate one of these for each MeshRenderBufferSet?
						FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
						FPrimitiveUniformShaderParametersBuilder Builder;
						BuildUniformShaderParameters(Builder);
						DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), Builder);

						if (BufferSet->IndexBuffer.Indices.Num() > 0)
						{
							DrawBatch(Collector, *BufferSet, BufferSet->IndexBuffer, CollisionMaterialInstance, bDrawWireframe, SDPG_World, ViewIndex, DynamicPrimitiveUniformBuffer);
						}
					}
				}
			}

			// Draw simple collision as wireframe if 'show collision', collision is enabled, and we are not using the complex as the simple
			const bool bDrawSimpleWireframeCollision = (EngineShowFlags.Collision && IsCollisionEnabled() && CollisionTraceFlag != ECollisionTraceFlag::CTF_UseComplexAsSimple);

			if((bDrawSimpleCollision || bDrawSimpleWireframeCollision))
			{
				if (ParentBaseComponent->GetBodySetup())
				{
					// Avoid zero scaling, otherwise GeomTransform below will assert
					if (FMath::Abs(GetLocalToWorld().Determinant()) > UE_SMALL_NUMBER)
					{
						const bool bDrawSolid = !bDrawSimpleWireframeCollision;

						if (AllowDebugViewmodes() && bDrawSolid)
						{
							// Make a material for drawing solid collision stuff
							FColoredMaterialRenderProxy* SolidMaterialInstance = new FColoredMaterialRenderProxy(
								GEngine->ShadedLevelColorationUnlitMaterial->GetRenderProxy(),
								GetWireframeColor()
							);

							Collector.RegisterOneFrameMaterialProxy(SolidMaterialInstance);

							FTransform GeomTransform(GetLocalToWorld());
							CachedAggGeom.GetAggGeom(GeomTransform, GetWireframeColor().ToFColor(true), SolidMaterialInstance, false, true, AlwaysHasVelocity(), ViewIndex, Collector);
						}
						// wireframe
						else
						{
							FTransform GeomTransform(GetLocalToWorld());
							CachedAggGeom.GetAggGeom(GeomTransform, GetSelectionColor(SimpleCollisionColor, bProxyIsSelected, IsHovered()).ToFColor(true), NULL, bOwnerIsNull, false, AlwaysHasVelocity(), ViewIndex, Collector);
						}

						// Note: if dynamic mesh component could have nav collision data, we'd also draw that here (see the similar code in StaticMeshRenderer.cpp)
					}
				}
			}
		}
	}
#endif // UE_ENABLE_DEBUG_DRAWING

}

void FBaseDynamicMeshSceneProxy::DrawBatch(FMeshElementCollector& Collector, const FMeshRenderBufferSet& RenderBuffers, const FDynamicMeshIndexBuffer32& IndexBuffer, FMaterialRenderProxy* UseMaterial, bool bWireframe, ESceneDepthPriorityGroup DepthPriority, int ViewIndex, FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer) const
{
	FMeshBatch& Mesh = Collector.AllocateMesh();
	FMeshBatchElement& BatchElement = Mesh.Elements[0];
	BatchElement.IndexBuffer = &IndexBuffer;
	Mesh.bWireframe = bWireframe;
	Mesh.bDisableBackfaceCulling = bTwoSided;
	Mesh.VertexFactory = &RenderBuffers.VertexFactory;
	Mesh.MaterialRenderProxy = UseMaterial;

	BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;

	BatchElement.FirstIndex = 0;
	BatchElement.NumPrimitives = IndexBuffer.Indices.Num() / 3;
	BatchElement.MinVertexIndex = 0;
	BatchElement.MaxVertexIndex = RenderBuffers.PositionVertexBuffer.GetNumVertices() - 1;
	Mesh.ReverseCulling = IsLocalToWorldDeterminantNegative();
	Mesh.Type = PT_TriangleList;
	Mesh.DepthPriorityGroup = DepthPriority;
	// if this is a wireframe draw pass then we do not want to apply View Mode Overrides
	Mesh.bCanApplyViewModeOverrides = (bWireframe) ? false : this->bEnableViewModeOverrides;

	for (ERuntimeVirtualTextureMaterialType MaterialType : RuntimeVirtualTextureMaterialTypes)
	{
		FMeshBatch& VirtualMesh = Collector.AllocateMesh();
		VirtualMesh = Mesh;
		VirtualMesh.CastShadow = 0;
		VirtualMesh.bUseAsOccluder = 0;
		VirtualMesh.bUseForDepthPass = 0;
		VirtualMesh.bUseForMaterial = 0;
		VirtualMesh.bDitheredLODTransition = 0;
		VirtualMesh.bRenderToVirtualTexture = 1;
		VirtualMesh.RuntimeVirtualTextureMaterialType = (uint32)MaterialType;
		Collector.AddMesh(ViewIndex, VirtualMesh);
	}
	
	Collector.AddMesh(ViewIndex, Mesh);
}

#if WITH_EDITOR
HHitProxy* FBaseDynamicMeshSceneProxy::CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy>>& OutHitProxies)
{
	return FBaseDynamicMeshSceneProxy::CreateHitProxies(Component->GetPrimitiveComponentInterface(), OutHitProxies);
}

HHitProxy* FBaseDynamicMeshSceneProxy::CreateHitProxies(IPrimitiveComponent* ComponentInterface, TArray<TRefCountPtr<HHitProxy>>& OutHitProxies)
{
	// Similar to a static mesh component, we call back into the component to ask it for a hit proxy. In our
	//  case, we don't have section indices to give it- we're just using this as an optional customization
	//  point for the dynamic mesh component to create a custom hit proxy without having to use a different
	//  scene proxy.
	HHitProxy* HitProxy = ComponentInterface->CreateMeshHitProxy(0, 0);
	if (HitProxy)
	{
		OutHitProxies.Add(HitProxy);
		return HitProxy;
	}
	
	// Otherwise fall back to base class implementation
	return FPrimitiveSceneProxy::CreateHitProxies(ComponentInterface, OutHitProxies);
}
#endif //WITH_EDITOR

bool FBaseDynamicMeshSceneProxy::AllowStaticDrawPath(const FSceneView* View) const
{
	bool bAllowDebugViews = AllowDebugViewmodes();
	if (!bAllowDebugViews)
	{
		return true;
	}
	const FEngineShowFlags& EngineShowFlags = View->Family->EngineShowFlags;
	bool bWantWireframeOnShaded = ParentBaseComponent->GetEnableWireframeRenderPass();
	bool bWireframe = EngineShowFlags.Wireframe || bWantWireframeOnShaded;
	if (bWireframe)
	{
		return false;
	}
	bool bDrawSimpleCollision = false, bDrawComplexCollision = false;
	bool bDrawCollisionView = IsCollisionView(EngineShowFlags, bDrawSimpleCollision, bDrawComplexCollision); // check for the full collision views
	bool bDrawCollisionFlags = EngineShowFlags.Collision && IsCollisionEnabled(); // check for single component collision rendering
	bool bDrawCollision = bDrawCollisionFlags || bDrawSimpleCollision || bDrawCollisionView;
	if (bDrawCollision)
	{
		return false;
	}
	bool bIsSelected = IsSelected();
	bool bColorOverrides = (bIsSelected && EngineShowFlags.VertexColors) || (ParentBaseComponent->ColorMode != EDynamicMeshComponentColorOverrideMode::None);
	return !bColorOverrides;
}


void FBaseDynamicMeshSceneProxy::DrawStaticElements(FStaticPrimitiveDrawInterface* PDI)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BaseDynamicMeshSceneProxy_DrawStaticElements);

	if (!bPreferStaticDrawPath)
	{
		return;
	}

	UMaterialInterface* UseSecondaryMaterial = nullptr;
	if (ParentBaseComponent->HasSecondaryRenderMaterial())
	{
		UseSecondaryMaterial = ParentBaseComponent->GetSecondaryRenderMaterial();
	}
	bool bDrawSecondaryBuffers = ParentBaseComponent->GetSecondaryBuffersVisibility();

	ESceneDepthPriorityGroup DepthPriority = SDPG_World;
	
	const int32 NumRuntimeVirtualTextureTypes = RuntimeVirtualTextureMaterialTypes.Num();

	TArray<FMeshRenderBufferSet*> Buffers;
	GetActiveRenderBufferSets(Buffers);
	PDI->ReserveMemoryForMeshes(Buffers.Num() * (1 + NumRuntimeVirtualTextureTypes));

	// Draw the mesh.
	int32 SectionIndexCounter = 0;
	for (FMeshRenderBufferSet* BufferSet : Buffers)
	{
		if (BufferSet->TriangleCount == 0)
		{
			continue;
		}

		UMaterialInterface* UseMaterial = BufferSet->Material;
		if (ParentBaseComponent->HasOverrideRenderMaterial(0))
		{
			UseMaterial = ParentBaseComponent->GetOverrideRenderMaterial(0);
		}
		FMaterialRenderProxy* MaterialProxy = UseMaterial->GetRenderProxy();

		// lock buffers so that they aren't modified while we are submitting them
		FScopeLock BuffersLock(&BufferSet->BuffersLock);

		FMeshBatch MeshBatch;

		FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
		BatchElement.IndexBuffer = &BufferSet->IndexBuffer;
		MeshBatch.VertexFactory = &BufferSet->VertexFactory;
		MeshBatch.MaterialRenderProxy = MaterialProxy;

		BatchElement.PrimitiveUniformBuffer = GetUniformBuffer();
		BatchElement.NumPrimitives = BufferSet->IndexBuffer.Indices.Num() / 3;
		BatchElement.FirstIndex = 0;
		BatchElement.MinVertexIndex = 0;
		BatchElement.MaxVertexIndex = BufferSet->PositionVertexBuffer.GetNumVertices() - 1;
		MeshBatch.ReverseCulling = IsLocalToWorldDeterminantNegative();
		MeshBatch.Type = PT_TriangleList;
		MeshBatch.DepthPriorityGroup = DepthPriority;
		MeshBatch.bCanApplyViewModeOverrides = this->bEnableViewModeOverrides;
		MeshBatch.LODIndex = 0;
		MeshBatch.SegmentIndex = SectionIndexCounter;
		MeshBatch.MeshIdInPrimitive = SectionIndexCounter;
		SectionIndexCounter++;

		MeshBatch.LCI = nullptr; // lightmap cache interface (allowed to be null)
		MeshBatch.CastShadow = true;
		MeshBatch.bUseForMaterial = true;
		MeshBatch.bDitheredLODTransition = false;
		MeshBatch.bUseForDepthPass = true;
		MeshBatch.bUseAsOccluder = ShouldUseAsOccluder();
		
		if (NumRuntimeVirtualTextureTypes > 0)
		{
			FMeshBatch VirtualMeshBatch(MeshBatch);
			VirtualMeshBatch.CastShadow = 0;
			VirtualMeshBatch.bUseAsOccluder = 0;
			VirtualMeshBatch.bUseForDepthPass = 0;
			VirtualMeshBatch.bUseForMaterial = 0;
			VirtualMeshBatch.bDitheredLODTransition = 0;
			VirtualMeshBatch.bRenderToVirtualTexture = 1;
			
			for (ERuntimeVirtualTextureMaterialType MaterialType : RuntimeVirtualTextureMaterialTypes)
			{
				VirtualMeshBatch.RuntimeVirtualTextureMaterialType = (uint32)MaterialType;
				PDI->DrawMesh(VirtualMeshBatch, FLT_MAX);
			}
		}

		PDI->DrawMesh(MeshBatch, FLT_MAX);
	}

}


void FBaseDynamicMeshSceneProxy::SetCollisionData()
{
#if UE_ENABLE_DEBUG_DRAWING
	FScopeLock Lock(&CachedCollisionLock);
	bHasCollisionData = true;
	bOwnerIsNull = ParentBaseComponent->GetOwner() == nullptr;
	bHasComplexMeshData = false;
	if (UBodySetup* BodySetup = ParentBaseComponent->GetBodySetup())
	{
		CollisionTraceFlag = BodySetup->GetCollisionTraceFlag();
		CachedAggGeom = BodySetup->AggGeom;
		
		if (IInterface_CollisionDataProvider* CDP = Cast<IInterface_CollisionDataProvider>(ParentBaseComponent))
		{
			bHasComplexMeshData = CDP->ContainsPhysicsTriMeshData(BodySetup->bMeshCollideAll);
		}
	}
	else
	{
		CachedAggGeom = FKAggregateGeom();
	}
	CollisionResponse = ParentBaseComponent->GetCollisionResponseToChannels();
#endif
}

#if RHI_RAYTRACING

bool FBaseDynamicMeshSceneProxy::IsRayTracingRelevant() const 
{
	return true;
}

bool FBaseDynamicMeshSceneProxy::HasRayTracingRepresentation() const
{
	return true;
}


void FBaseDynamicMeshSceneProxy::GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_BaseDynamicMeshSceneProxy_GetDynamicRayTracingInstances);

	ESceneDepthPriorityGroup DepthPriority = SDPG_World;

	TArray<FMeshRenderBufferSet*> Buffers;
	GetActiveRenderBufferSets(Buffers);

	// will use this material instead of any others below, if it becomes non-null
	UMaterialInterface* ForceOverrideMaterial = nullptr;
	const bool bVertexColor = ParentBaseComponent->ColorMode == EDynamicMeshComponentColorOverrideMode::VertexColors ||
		ParentBaseComponent->ColorMode == EDynamicMeshComponentColorOverrideMode::Polygroups ||
		ParentBaseComponent->ColorMode == EDynamicMeshComponentColorOverrideMode::Constant;
	if (bVertexColor)
	{
		ForceOverrideMaterial = UBaseDynamicMeshComponent::GetDefaultVertexColorMaterial_RenderThread();
	}

	UMaterialInterface* UseSecondaryMaterial = ForceOverrideMaterial;
	if (ParentBaseComponent->HasSecondaryRenderMaterial() && ForceOverrideMaterial == nullptr)
	{
		UseSecondaryMaterial = ParentBaseComponent->GetSecondaryRenderMaterial();
	}
	bool bDrawSecondaryBuffers = ParentBaseComponent->GetSecondaryBuffersVisibility();

	// is it safe to share this between primary and secondary raytracing batches?
	FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer = Collector.AllocateOneFrameResource<FDynamicPrimitiveUniformBuffer>();
	FPrimitiveUniformShaderParametersBuilder Builder;
	BuildUniformShaderParameters(Builder);
	DynamicPrimitiveUniformBuffer.Set(Collector.GetRHICommandList(), Builder);

	// Draw the active buffer sets
	for (FMeshRenderBufferSet* BufferSet : Buffers)
	{
		UMaterialInterface* UseMaterial = BufferSet->Material;
		if (ParentBaseComponent->HasOverrideRenderMaterial(0))
		{
			UseMaterial = ParentBaseComponent->GetOverrideRenderMaterial(0);
		}
		if (ForceOverrideMaterial)
		{
			UseMaterial = ForceOverrideMaterial;
		}
		FMaterialRenderProxy* MaterialProxy = UseMaterial->GetRenderProxy();

		if (BufferSet->TriangleCount == 0)
		{
			continue;
		}
		if (BufferSet->bIsRayTracingDataValid == false)
		{
			continue;
		}

		// Lock buffers so that they aren't modified while we are submitting them.
		FScopeLock BuffersLock(&BufferSet->BuffersLock);

		// draw primary index buffer
		if (BufferSet->IndexBuffer.Indices.Num() > 0
			&& BufferSet->PrimaryRayTracingGeometry.IsValid())
		{
			ensure(BufferSet->PrimaryRayTracingGeometry.GetInitializer().IndexBuffer.IsValid());
			DrawRayTracingBatch(Collector, *BufferSet, BufferSet->IndexBuffer, BufferSet->PrimaryRayTracingGeometry, MaterialProxy, DepthPriority, DynamicPrimitiveUniformBuffer);
		}

		// draw secondary index buffer if we have it, falling back to base material if we don't have the Secondary material
		FMaterialRenderProxy* UseSecondaryMaterialProxy = (UseSecondaryMaterial != nullptr) ? UseSecondaryMaterial->GetRenderProxy() : MaterialProxy;
		if (bDrawSecondaryBuffers
			&& BufferSet->SecondaryIndexBuffer.Indices.Num() > 0
			&& UseSecondaryMaterialProxy != nullptr
			&& BufferSet->SecondaryRayTracingGeometry.IsValid())
		{
			ensure(BufferSet->SecondaryRayTracingGeometry.GetInitializer().IndexBuffer.IsValid());
			DrawRayTracingBatch(Collector, *BufferSet, BufferSet->SecondaryIndexBuffer, BufferSet->SecondaryRayTracingGeometry, UseSecondaryMaterialProxy, DepthPriority, DynamicPrimitiveUniformBuffer);
		}
	}
}

void FBaseDynamicMeshSceneProxy::DrawRayTracingBatch(FRayTracingInstanceCollector& Collector, const FMeshRenderBufferSet& RenderBuffers, const FDynamicMeshIndexBuffer32& IndexBuffer, FRayTracingGeometry& RayTracingGeometry, FMaterialRenderProxy* UseMaterialProxy, ESceneDepthPriorityGroup DepthPriority, FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer) const
{
	ensure(RayTracingGeometry.GetInitializer().IndexBuffer.IsValid());

	TConstArrayView<const FSceneView*> Views = Collector.GetViews();
	const uint32 VisibilityMap = Collector.GetVisibilityMap();

	// RT geometry will be generated based on first active view and then reused for all other views
	// TODO: Expose a way for developers to control whether to reuse RT geometry or create one per-view
	const int32 FirstActiveViewIndex = FMath::CountTrailingZeros(VisibilityMap);
	checkf(Views.IsValidIndex(FirstActiveViewIndex), TEXT("There should be at least one active view when calling GetDynamicRayTracingInstances(...)."));

	const FSceneView* FirstActiveView = Views[FirstActiveViewIndex];

	FRayTracingInstance RayTracingInstance;
	RayTracingInstance.Geometry = &RayTracingGeometry;
	RayTracingInstance.InstanceTransforms.Add(GetLocalToWorld());

	uint32 SectionIdx = 0;
	FMeshBatch MeshBatch;

	MeshBatch.VertexFactory = &RenderBuffers.VertexFactory;
	MeshBatch.SegmentIndex = 0;
	MeshBatch.MaterialRenderProxy = UseMaterialProxy;
	MeshBatch.Type = PT_TriangleList;
	MeshBatch.DepthPriorityGroup = DepthPriority;
	MeshBatch.bCanApplyViewModeOverrides = this->bEnableViewModeOverrides;
	MeshBatch.CastRayTracedShadow = IsShadowCast(FirstActiveView);

	FMeshBatchElement& BatchElement = MeshBatch.Elements[0];
	BatchElement.IndexBuffer = &IndexBuffer;
	BatchElement.PrimitiveUniformBufferResource = &DynamicPrimitiveUniformBuffer.UniformBuffer;
	BatchElement.FirstIndex = 0;
	BatchElement.NumPrimitives = IndexBuffer.Indices.Num() / 3;
	BatchElement.MinVertexIndex = 0;
	BatchElement.MaxVertexIndex = RenderBuffers.PositionVertexBuffer.GetNumVertices() - 1;

	RayTracingInstance.Materials.Add(MeshBatch);

	for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
	{
		if ((VisibilityMap & (1 << ViewIndex)) == 0)
		{
			continue;
		}

		Collector.AddRayTracingInstance(ViewIndex, RayTracingInstance);
	}
}

#endif // RHI_RAYTRACING






const FCardRepresentationData* FBaseDynamicMeshSceneProxy::GetMeshCardRepresentation() const
{
	return MeshCards.Get();
}


void FBaseDynamicMeshSceneProxy::UpdateLumenCardsFromBounds()
{
	if (!bVisibleInLumenScene || !UE::DynamicMesh::AllowLumenCardGeneration())
	{
		MeshCards.Reset();
		return;
	}

	FBox Box = ParentBaseComponent->GetLocalBounds().GetBox();

	if (MeshCards.IsValid() == false)
	{
		MeshCards = MakePimpl<FCardRepresentationData>();
	}

	*MeshCards = FCardRepresentationData();
	FMeshCardsBuildData& CardData = MeshCards->MeshCardsBuildData;

	CardData.Bounds = Box;
	// Mark as two-sided so a high sampling bias is used and hits are accepted even if they don't match well
	CardData.bMostlyTwoSided = true;

	MeshCardRepresentation::SetCardsFromBounds(CardData);
}



TUniquePtr<FDistanceFieldVolumeData> FBaseDynamicMeshSceneProxy::ComputeDistanceFieldForMesh(
	const FDynamicMesh3& Mesh, 
	FProgressCancel& Progress,
	float DistanceFieldResolutionScale, 
	bool bGenerateAsIfTwoSided)
{
	return TUniquePtr<FDistanceFieldVolumeData>();
}

void FBaseDynamicMeshSceneProxy::SetNewDistanceField(TSharedPtr<FDistanceFieldVolumeData> NewDistanceField, bool bInInitialize)
{
	ensureMsgf(false, TEXT("Distance fields not supported"));
}





