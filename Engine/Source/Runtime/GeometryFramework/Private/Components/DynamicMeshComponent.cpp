// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Components/DynamicMeshComponent.h"
#include "PrimitiveSceneProxy.h"
#include "MaterialShared.h"
#include "Engine/CollisionProfile.h"
#include "Engine/World.h"
#include "Materials/Material.h"
#include "Async/Async.h"
#include "Engine/CollisionProfile.h"
#include "PhysicsEngine/PhysicsSettings.h"

#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/MeshNormals.h"
#include "MeshDescriptionToDynamicMesh.h"
#include "Util/ColorConstants.h"

#include "Changes/MeshVertexChange.h"
#include "Changes/MeshChange.h"
#include "DynamicMesh/MeshTransforms.h"

#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "UObject/UObjectGlobals.h"
#include <atomic>

// default proxy for this component
#include "Components/DynamicMeshSceneProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMeshComponent)

using namespace UE::Geometry;


static TAutoConsoleVariable<int32> CVarDynamicMeshComponent_MaxComplexCollisionTriCount(
	TEXT("geometry.DynamicMesh.MaxComplexCollisionTriCount"),
	250000,
	TEXT("If a DynamicMeshCompnent's UDynamicMesh has a larger triangle count than this value, it will not be passed to the Physics system to be used as Complex Collision geometry. A negative value indicates no limit.")
);

static TAutoConsoleVariable<bool> CVarDynamicMeshComponent_AllowCollisionAsyncBuildInEditor(
	TEXT("geometry.DynamicMesh.AllowAsyncCollisionBuildInEditor"),
	false,
	TEXT("If enabled, we allow async build of dynamic mesh component collision in (non-PIE) Editor. Otherwise, async collision build will only happen in game worlds (PIE/runtime).")
);


namespace
{
	// probably should be something defined for the whole tool framework...
#if WITH_EDITOR
	static EAsyncExecution DynamicMeshComponentAsyncExecTarget = EAsyncExecution::LargeThreadPool;
#else
	static EAsyncExecution DynamicMeshComponentAsyncExecTarget = EAsyncExecution::ThreadPool;
#endif
}



namespace UELocal
{
	static EMeshRenderAttributeFlags ConvertChangeFlagsToUpdateFlags(EDynamicMeshAttributeChangeFlags ChangeFlags)
	{
		EMeshRenderAttributeFlags UpdateFlags = EMeshRenderAttributeFlags::None;
		if ((ChangeFlags & EDynamicMeshAttributeChangeFlags::VertexPositions) != EDynamicMeshAttributeChangeFlags::Unknown)
		{
			UpdateFlags |= EMeshRenderAttributeFlags::Positions;
		}
		if ((ChangeFlags & EDynamicMeshAttributeChangeFlags::NormalsTangents) != EDynamicMeshAttributeChangeFlags::Unknown)
		{
			UpdateFlags |= EMeshRenderAttributeFlags::VertexNormals;
		}
		if ((ChangeFlags & EDynamicMeshAttributeChangeFlags::VertexColors) != EDynamicMeshAttributeChangeFlags::Unknown)
		{
			UpdateFlags |= EMeshRenderAttributeFlags::VertexColors;
		}
		if ((ChangeFlags & EDynamicMeshAttributeChangeFlags::UVs) != EDynamicMeshAttributeChangeFlags::Unknown)
		{
			UpdateFlags |= EMeshRenderAttributeFlags::VertexUVs;
		}
		return UpdateFlags;
	}

}



UDynamicMeshComponent::UDynamicMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	PrimaryComponentTick.bCanEverTick = false;

	SetCollisionProfileName(UCollisionProfile::NoCollision_ProfileName);

	MeshObject = CreateDefaultSubobject<UDynamicMesh>(TEXT("DynamicMesh"));
	//MeshObject->SetFlags(RF_Transactional);

	MeshObjectChangedHandle = MeshObject->OnMeshChanged().AddUObject(this, &UDynamicMeshComponent::OnMeshObjectChanged);

	ResetProxy();
}

void UDynamicMeshComponent::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
}

void UDynamicMeshComponent::PostLoad()
{
	Super::PostLoad();

	const int32 UE5ReleaseStreamObjectVersion = GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	if (UE5ReleaseStreamObjectVersion < FUE5ReleaseStreamObjectVersion::DynamicMeshComponentsDefaultUseExternalTangents)
	{
		// Set the old default value
		if (TangentsType == EDynamicMeshComponentTangentsMode::Default)
		{
			TangentsType = EDynamicMeshComponentTangentsMode::NoTangents;
		}
	}

	// The intention here is that MeshObject is never nullptr, however we cannot guarantee this as a subclass
	// may have set it to null, and/or some type of serialization issue has caused it to fail to save/load.
	// Avoid immediate crashes by creating a new UDynamicMesh here in such cases
	if (ensure(MeshObject != nullptr) == false)
	{
		MeshObject = NewObject<UDynamicMesh>(this, TEXT("DynamicMesh"));
	}

	MeshObjectChangedHandle = MeshObject->OnMeshChanged().AddUObject(this, &UDynamicMeshComponent::OnMeshObjectChanged);

	ResetProxy();

	// This is a fixup for existing UDynamicMeshComponents that did not have the correct flags 
	// on the Instanced UBodySetup, these flags are now set in GetBodySetup() on new instances
	if (MeshBodySetup && IsTemplate())
	{
		MeshBodySetup->SetFlags(RF_Public | RF_ArchetypeObject);
	}

	// make sure BodySetup is created
	GetBodySetup();
}

void UDynamicMeshComponent::PostEditImport()
{
	Super::PostEditImport();
	
	// MeshObject should never be null here, but we re-validate that it isn't (similar to PostLoad method, above)
	if (ensure(MeshObject != nullptr) == false)
	{
		MeshObject = NewObject<UDynamicMesh>(this, TEXT("DynamicMesh"));
		MeshObjectChangedHandle = MeshObject->OnMeshChanged().AddUObject(this, &UDynamicMeshComponent::OnMeshObjectChanged);
	}
}

#if WITH_EDITOR
void UDynamicMeshComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropName = PropertyChangedEvent.GetPropertyName();
	if (PropName == GET_MEMBER_NAME_CHECKED(UDynamicMeshComponent, TangentsType))
	{
		InvalidateAutoCalculatedTangents();
	}
	else if ( (PropName == GET_MEMBER_NAME_CHECKED(UDynamicMeshComponent, bEnableComplexCollision)) ||
		(PropName == GET_MEMBER_NAME_CHECKED(UDynamicMeshComponent, CollisionType)) ||
		(PropName == GET_MEMBER_NAME_CHECKED(UDynamicMeshComponent, bDeferCollisionUpdates))  )
	{
		if (bDeferCollisionUpdates)
		{
			InvalidatePhysicsData();
		}
		else
		{
			RebuildPhysicsData();
		}
	}
}
#endif


void UDynamicMeshComponent::SetMesh(UE::Geometry::FDynamicMesh3&& MoveMesh)
{
	if (ensure(MeshObject) && ensureMsgf(IsEditable(), TEXT("Attempted to modify the internal mesh of a UDynamicMeshComponent that is not editable")))
	{
		MeshObject->SetMesh(MoveTemp(MoveMesh));
	}
}


void UDynamicMeshComponent::ProcessMesh(
	TFunctionRef<void(const UE::Geometry::FDynamicMesh3&)> ProcessFunc ) const
{
	if (MeshObject)
	{
		MeshObject->ProcessMesh(ProcessFunc);
	}
}


void UDynamicMeshComponent::EditMesh(TFunctionRef<void(UE::Geometry::FDynamicMesh3&)> EditFunc,
										   EDynamicMeshComponentRenderUpdateMode UpdateMode,
										   EDynamicMeshChangeType ChangeType,
										   EDynamicMeshAttributeChangeFlags ChangeFlags)
{
	if (MeshObject && ensureMsgf(IsEditable(), TEXT("Attempted to modify the internal mesh of a UDynamicMeshComponent that is not editable")))
	{
		MeshObject->EditMesh(EditFunc, ChangeType, ChangeFlags);
		if (UpdateMode != EDynamicMeshComponentRenderUpdateMode::NoUpdate)
		{
			NotifyMeshUpdated();
		}
	}
}


void UDynamicMeshComponent::SetRenderMeshPostProcessor(TUniquePtr<IRenderMeshPostProcessor> Processor)
{
	if (!ensure(MeshObject))
	{
		return;
	}

	RenderMeshPostProcessor = MoveTemp(Processor);
	if (RenderMeshPostProcessor)
	{
		if (!RenderMesh)
		{
			RenderMesh = MakeUnique<FDynamicMesh3>(*GetMesh());
		}
	}
	else
	{
		// No post processor, no render mesh
		RenderMesh = nullptr;
	}
}

FDynamicMesh3* UDynamicMeshComponent::GetRenderMesh()
{
	if (RenderMeshPostProcessor && RenderMesh)
	{
		return RenderMesh.Get();
	}
	else
	{
		return GetMesh();
	}
}

const FDynamicMesh3* UDynamicMeshComponent::GetRenderMesh() const
{
	if (RenderMeshPostProcessor && RenderMesh)
	{
		return RenderMesh.Get();
	}
	else
	{
		return GetMesh();
	}
}




void UDynamicMeshComponent::ApplyTransform(const FTransform3d& Transform, bool bInvert)
{
	if (ensure(MeshObject) && ensureMsgf(IsEditable(), TEXT("Attempted to modify the internal mesh of a UDynamicMeshComponent that is not editable")))
	{
		MeshObject->EditMesh([&](FDynamicMesh3& EditMesh)
		{
			if (bInvert)
			{
				MeshTransforms::ApplyTransformInverse(EditMesh, Transform, true);
			}
			else
			{
				MeshTransforms::ApplyTransform(EditMesh, Transform, true);
			}
		}, EDynamicMeshChangeType::DeformationEdit, EDynamicMeshAttributeChangeFlags::VertexPositions | EDynamicMeshAttributeChangeFlags::NormalsTangents, /*bDeferChangeEvents*/ false);
	}
}



bool UDynamicMeshComponent::ValidateMaterialSlots(bool bCreateIfMissing, bool bDeleteExtraSlots)
{
	int32 MaxMeshMaterialID = 0;
	ProcessMesh([&](const FDynamicMesh3& EditMesh)
	{
		if (EditMesh.HasAttributes() && EditMesh.Attributes()->HasMaterialID() && EditMesh.Attributes()->GetMaterialID() != nullptr)
		{
			const FDynamicMeshMaterialAttribute* MaterialIDs = EditMesh.Attributes()->GetMaterialID();
			for (int TriangleID : EditMesh.TriangleIndicesItr())
			{
				MaxMeshMaterialID = FMath::Max(MaxMeshMaterialID, MaterialIDs->GetValue(TriangleID));
			}
		}
	});
	int32 NumRequiredMaterials = MaxMeshMaterialID + 1;

	int32 NumMaterials = GetNumMaterials();
	if ( bCreateIfMissing && NumMaterials < NumRequiredMaterials )
	{
		for (int32 k = NumMaterials; k < NumRequiredMaterials; ++k)
		{
			SetMaterial(k, nullptr);
		}
	}
	NumMaterials = GetNumMaterials();

	if (bDeleteExtraSlots && NumMaterials > NumRequiredMaterials)
	{
		SetNumMaterials(NumRequiredMaterials);
	}
	NumMaterials = GetNumMaterials();

	return (NumMaterials == NumRequiredMaterials);
}


void UDynamicMeshComponent::ConfigureMaterialSet(const TArray<UMaterialInterface*>& NewMaterialSet, bool bDeleteExtraSlots)
{
	for (int k = 0; k < NewMaterialSet.Num(); ++k)
	{
		SetMaterial(k, NewMaterialSet[k]);
	}	
	if (bDeleteExtraSlots)
	{
		SetNumMaterials(NewMaterialSet.Num());
	}
}


void UDynamicMeshComponent::SetTangentsType(EDynamicMeshComponentTangentsMode NewTangentsType)
{
	if (NewTangentsType != TangentsType)
	{
		TangentsType = NewTangentsType;
		InvalidateAutoCalculatedTangents();
	}
}

void UDynamicMeshComponent::InvalidateAutoCalculatedTangents() 
{ 
	bAutoCalculatedTangentsValid = false; 
}

const UE::Geometry::FMeshTangentsf* UDynamicMeshComponent::GetAutoCalculatedTangents() 
{ 
	if (ensure(MeshObject) && GetTangentsType() == EDynamicMeshComponentTangentsMode::AutoCalculated && GetDynamicMesh()->GetMeshRef().HasAttributes())
	{
		UpdateAutoCalculatedTangents();
		return (bAutoCalculatedTangentsValid) ? &AutoCalculatedTangents : nullptr;
	}
	return nullptr;
}

void UDynamicMeshComponent::UpdateAutoCalculatedTangents()
{
	if (GetTangentsType() == EDynamicMeshComponentTangentsMode::AutoCalculated && bAutoCalculatedTangentsValid == false)
	{
		GetDynamicMesh()->ProcessMesh([&](const FDynamicMesh3& Mesh)
		{
			if (Mesh.HasAttributes())
			{
				const FDynamicMeshUVOverlay* UVOverlay = Mesh.Attributes()->PrimaryUV();
				const FDynamicMeshNormalOverlay* NormalOverlay = Mesh.Attributes()->PrimaryNormals();
				if (UVOverlay && NormalOverlay)
				{
					AutoCalculatedTangents.SetMesh(&Mesh);
					AutoCalculatedTangents.ComputeTriVertexTangents(NormalOverlay, UVOverlay, FComputeTangentsOptions());
					AutoCalculatedTangents.SetMesh(nullptr);
					bAutoCalculatedTangentsValid = true;
				}
			}
		});
	}
}




void UDynamicMeshComponent::UpdateLocalBounds()
{
	LocalBounds = MeshObject ? GetMesh()->GetBounds(true) : FAxisAlignedBox3d::Empty();
	if (LocalBounds.MaxDim() <= 0)
	{
		// If bbox is empty, set a very small bbox to avoid log spam/etc in other engine systems.
		// The check used is generally IsNearlyZero(), which defaults to KINDA_SMALL_NUMBER, so set 
		// a slightly larger box here to be above that threshold
		LocalBounds = FAxisAlignedBox3d(FVector3d::Zero(), (double)(KINDA_SMALL_NUMBER + SMALL_NUMBER) );
	}
}

FDynamicMeshSceneProxy* UDynamicMeshComponent::GetCurrentSceneProxy() 
{ 
	if (bProxyValid)
	{
		return (FDynamicMeshSceneProxy*)SceneProxy;
	}
	return nullptr;
}


void UDynamicMeshComponent::ResetProxy()
{
	bProxyValid = false;
	InvalidateAutoCalculatedTangents();

	// Need to recreate scene proxy to send it over
	MarkRenderStateDirty();
	UpdateLocalBounds();
	UpdateBounds();

	// this seems speculative, ie we may not actually have a mesh update, but we currently ResetProxy() in lots
	// of places where that is what it means
	GetDynamicMesh()->PostRealtimeUpdate();
}

void UDynamicMeshComponent::NotifyMeshUpdated()
{
	if (MeshObject && RenderMeshPostProcessor)
	{
		RenderMeshPostProcessor->ProcessMesh(*GetMesh(), *RenderMesh);
	}

	ResetProxy();
}

void UDynamicMeshComponent::NotifyMeshModified()
{
	NotifyMeshUpdated();
}


void UDynamicMeshComponent::FastNotifyColorsUpdated()
{
	if (!ensure(MeshObject))
	{
		return;
	}

	// should not be using fast paths if we have to run mesh postprocessor
	if (ensure(!RenderMeshPostProcessor) == false)
	{
		RenderMeshPostProcessor->ProcessMesh(*GetMesh(), *RenderMesh);
		ResetProxy();
		return;
	}

	FDynamicMeshSceneProxy* Proxy = GetCurrentSceneProxy();
	if (Proxy && AllowFastUpdate())
	{
		if (HasTriangleColorFunction() && Proxy->MeshRenderBufferSetConverter.bUsePerTriangleColor == false )
		{
			Proxy->MeshRenderBufferSetConverter.bUsePerTriangleColor = true;
			Proxy->MeshRenderBufferSetConverter.PerTriangleColorFunc = [this](const FDynamicMesh3* MeshIn, int TriangleID) { return GetTriangleColor(MeshIn, TriangleID); };
		} 
		else if ( !HasTriangleColorFunction() && Proxy->MeshRenderBufferSetConverter.bUsePerTriangleColor == true)
		{
			Proxy->MeshRenderBufferSetConverter.bUsePerTriangleColor = false;
			Proxy->MeshRenderBufferSetConverter.PerTriangleColorFunc = nullptr;
		}

		if (HasVertexColorRemappingFunction() && Proxy->MeshRenderBufferSetConverter.bApplyVertexColorRemapping == false)
		{
			Proxy->MeshRenderBufferSetConverter.bApplyVertexColorRemapping = true;
			Proxy->MeshRenderBufferSetConverter.VertexColorRemappingFunc = [this](FVector4f& Color) { RemapVertexColor(Color); };
		}
		else if (!HasVertexColorRemappingFunction() && Proxy->MeshRenderBufferSetConverter.bApplyVertexColorRemapping == true)
		{
			Proxy->MeshRenderBufferSetConverter.bApplyVertexColorRemapping = false;
			Proxy->MeshRenderBufferSetConverter.VertexColorRemappingFunc = nullptr;
		}

		Proxy->FastUpdateVertices(false, false, true, false);
		//MarkRenderDynamicDataDirty();
	}
	else
	{
		ResetProxy();
	}
}



void UDynamicMeshComponent::FastNotifyPositionsUpdated(bool bNormals, bool bColors, bool bUVs)
{
	if (!ensure(MeshObject))
	{
		return;
	}

	// should not be using fast paths if we have to run mesh postprocessor
	if (ensure(!RenderMeshPostProcessor) == false)
	{
		RenderMeshPostProcessor->ProcessMesh(*GetMesh(), *RenderMesh);
		ResetProxy();
		return;
	}

	FDynamicMeshSceneProxy* Proxy = GetCurrentSceneProxy();
	if (Proxy && AllowFastUpdate())
	{
		// calculate bounds while we are updating vertices
		TFuture<void> UpdateBoundsCalc;
		UpdateBoundsCalc = Async(DynamicMeshComponentAsyncExecTarget, [this]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SimpleDynamicMeshComponent_FastPositionsUpdate_AsyncBoundsUpdate);
			UpdateLocalBounds();
		});

		GetCurrentSceneProxy()->FastUpdateVertices(true, bNormals, bColors, bUVs);

		//MarkRenderDynamicDataDirty();
		MarkRenderTransformDirty();
		UpdateBoundsCalc.Wait();
		UpdateBounds();

		GetDynamicMesh()->PostRealtimeUpdate();
	}
	else
	{
		ResetProxy();
	}
}


void UDynamicMeshComponent::FastNotifyVertexAttributesUpdated(bool bNormals, bool bColors, bool bUVs)
{
	if (!ensure(MeshObject))
	{
		return;
	}

	// should not be using fast paths if we have to run mesh postprocessor
	if (ensure(!RenderMeshPostProcessor) == false)
	{
		RenderMeshPostProcessor->ProcessMesh(*GetMesh(), *RenderMesh);
		ResetProxy();
		return;
	}

	FDynamicMeshSceneProxy* Proxy = GetCurrentSceneProxy();
	if (Proxy && ensure(bNormals || bColors || bUVs) && AllowFastUpdate())
	{
		GetCurrentSceneProxy()->FastUpdateVertices(false, bNormals, bColors, bUVs);
		//MarkRenderDynamicDataDirty();
		//MarkRenderTransformDirty();

		GetDynamicMesh()->PostRealtimeUpdate();
	}
	else
	{
		ResetProxy();
	}
}


void UDynamicMeshComponent::FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags UpdatedAttributes)
{
	if (!ensure(MeshObject))
	{
		return;
	}

	// should not be using fast paths if we have to run mesh postprocessor
	if (ensure(!RenderMeshPostProcessor) == false)
	{
		RenderMeshPostProcessor->ProcessMesh(*GetMesh(), *RenderMesh);
		ResetProxy();
		return;
	}

	FDynamicMeshSceneProxy* Proxy = GetCurrentSceneProxy();
	if (Proxy && ensure(UpdatedAttributes != EMeshRenderAttributeFlags::None) && AllowFastUpdate())
	{
		bool bPositions = (UpdatedAttributes & EMeshRenderAttributeFlags::Positions) != EMeshRenderAttributeFlags::None;

		// calculate bounds while we are updating vertices
		TFuture<void> UpdateBoundsCalc;
		if (bPositions)
		{
			UpdateBoundsCalc = Async(DynamicMeshComponentAsyncExecTarget, [this]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(SimpleDynamicMeshComponent_FastVertexAttribUpdate_AsyncBoundsUpdate);
				UpdateLocalBounds();
			});
		}

		GetCurrentSceneProxy()->FastUpdateVertices(bPositions,
			(UpdatedAttributes & EMeshRenderAttributeFlags::VertexNormals) != EMeshRenderAttributeFlags::None,
			(UpdatedAttributes & EMeshRenderAttributeFlags::VertexColors) != EMeshRenderAttributeFlags::None,
			(UpdatedAttributes & EMeshRenderAttributeFlags::VertexUVs) != EMeshRenderAttributeFlags::None);

		if (bPositions)
		{
			MarkRenderTransformDirty();
			UpdateBoundsCalc.Wait();
			UpdateBounds();
		}

		GetDynamicMesh()->PostRealtimeUpdate();
	}
	else
	{
		ResetProxy();
	}
}

void UDynamicMeshComponent::FastNotifyUVsUpdated()
{
	FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags::VertexUVs);
}


void UDynamicMeshComponent::NotifyMeshVertexAttributesModified( bool bPositions, bool bNormals, bool bUVs, bool bColors )
{
	EMeshRenderAttributeFlags Flags = EMeshRenderAttributeFlags::None;
	if (bPositions)
	{
		Flags |= EMeshRenderAttributeFlags::Positions;
	}
	if (bNormals)
	{
		Flags |= EMeshRenderAttributeFlags::VertexNormals;
	}
	if (bUVs)
	{
		Flags |= EMeshRenderAttributeFlags::VertexUVs;
	}
	if (bColors)
	{
		Flags |= EMeshRenderAttributeFlags::VertexColors;
	}

	if (Flags == EMeshRenderAttributeFlags::None)
	{
		return;
	}
	FastNotifyVertexAttributesUpdated(Flags);
}



void UDynamicMeshComponent::FastNotifySecondaryTrianglesChanged()
{
	if (!ensure(MeshObject))
	{
		return;
	}

	// should not be using fast paths if we have to run mesh postprocessor
	if (ensure(!RenderMeshPostProcessor) == false)
	{
		RenderMeshPostProcessor->ProcessMesh(*GetMesh(), *RenderMesh);
		ResetProxy();
		return;
	}

	FDynamicMeshSceneProxy* Proxy = GetCurrentSceneProxy();
	if (Proxy && AllowFastUpdate())
	{
		GetCurrentSceneProxy()->FastUpdateAllIndexBuffers();
		GetDynamicMesh()->PostRealtimeUpdate();
	}
	else
	{
		ResetProxy();
	}
}


void UDynamicMeshComponent::FastNotifyTriangleVerticesUpdated(const TArray<int32>& Triangles, EMeshRenderAttributeFlags UpdatedAttributes)
{
	FastNotifyDecompositionSetsUpdated([this, &Triangles]() -> TArray<int32>
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SimpleDynamicMeshComponent_FastTriangleVertexUpdate_FindSets);
		// compute list of sets to update
		TArray<int32> UpdatedSets;	
		for (int32 tid : Triangles)
		{
			int32 SetID = Decomposition->GetGroupForTriangle(tid);
			UpdatedSets.AddUnique(SetID);
		}
		return UpdatedSets;
	}, UpdatedAttributes);
}


void UDynamicMeshComponent::FastNotifyTriangleVerticesUpdated(const TSet<int32>& Triangles, EMeshRenderAttributeFlags UpdatedAttributes)
{
	FastNotifyDecompositionSetsUpdated([this, &Triangles]() -> TArray<int32>
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SimpleDynamicMeshComponent_FastTriangleVertexUpdate_FindSets);
		// compute list of sets to update
		TArray<int32> UpdatedSets;
		for (int32 tid : Triangles)
		{
			int32 SetID = Decomposition->GetGroupForTriangle(tid);
			UpdatedSets.AddUnique(SetID);
		}
		return UpdatedSets;
	}, UpdatedAttributes);
}

void UDynamicMeshComponent::FastNotifyVerticesUpdated(const TArray<int32>& Vertices, EMeshRenderAttributeFlags UpdatedAttributes)
{
	FastNotifyDecompositionSetsUpdated([this, &Vertices]() -> TArray<int32>
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SimpleDynamicMeshComponent_FastVertexUpdate_FindSets);
		// compute list of sets to update
		TArray<int32> UpdatedBuffers;
		int32 NumBuffers = Decomposition->Num();
		TArray<std::atomic<bool>> BufferFlags;
		BufferFlags.SetNum(NumBuffers);
		for (int32 k = 0; k < NumBuffers; ++k)
		{
			BufferFlags[k] = false;
		}
		const FDynamicMesh3* Mesh = GetMesh();
		check(Mesh);
		ParallelFor(Vertices.Num(), [&](int32 k)
		{
			int32 VID = Vertices[k];
			Mesh->EnumerateVertexTriangles(VID, [this, &BufferFlags](int32 TID)
			{
				int32 SetID = Decomposition->GetGroupForTriangle(TID);
				BufferFlags[SetID].store(true, std::memory_order_relaxed);
			});
		});
		for (int32 k = 0; k < NumBuffers; ++k)
		{
			if (BufferFlags[k])
			{
				UpdatedBuffers.Add(k);
			}
		}
		return UpdatedBuffers;
	}, UpdatedAttributes);
}

void UDynamicMeshComponent::FastNotifyDecompositionSetsUpdated(TFunctionRef<TArray<int32>(void)> GetUpdatedDecompositionIndices, EMeshRenderAttributeFlags UpdatedAttributes)
{
	if (!ensure(MeshObject))
	{
		return;
	}

	// should not be using fast paths if we have to run mesh postprocessor
	if (ensure(!RenderMeshPostProcessor) == false)
	{
		RenderMeshPostProcessor->ProcessMesh(*GetMesh(), *RenderMesh);
		ResetProxy();
		return;
	}

	bool bUpdateSecondarySort = (SecondaryTriFilterFunc) &&
		((UpdatedAttributes & EMeshRenderAttributeFlags::SecondaryIndexBuffers) != EMeshRenderAttributeFlags::None);

	FDynamicMeshSceneProxy* Proxy = GetCurrentSceneProxy();
	if (!Proxy || !AllowFastUpdate())
	{
		ResetProxy();
	}
	else if (!Decomposition)
	{
		FastNotifyVertexAttributesUpdated(UpdatedAttributes);
		if (bUpdateSecondarySort)
		{
			Proxy->FastUpdateAllIndexBuffers();
		}
		GetDynamicMesh()->PostRealtimeUpdate();
	}
	else
	{
		// compute list of sets to update
		TArray<int32> UpdatedSets = GetUpdatedDecompositionIndices();

		bool bPositions = (UpdatedAttributes & EMeshRenderAttributeFlags::Positions) != EMeshRenderAttributeFlags::None;

		// calculate bounds while we are updating vertices
		TFuture<void> UpdateBoundsCalc;
		if (bPositions)
		{
			UpdateBoundsCalc = Async(DynamicMeshComponentAsyncExecTarget, [this]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(SimpleDynamicMeshComponent_FastROIUpdate_AsyncBoundsUpdate);
				UpdateLocalBounds();
			});
		}

		// update the render buffers
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SimpleDynamicMeshComponent_FastROIUpdate_ApplyUpdate);
			Proxy->FastUpdateVertices(UpdatedSets, bPositions,
				(UpdatedAttributes & EMeshRenderAttributeFlags::VertexNormals) != EMeshRenderAttributeFlags::None,
				(UpdatedAttributes & EMeshRenderAttributeFlags::VertexColors) != EMeshRenderAttributeFlags::None,
				(UpdatedAttributes & EMeshRenderAttributeFlags::VertexUVs) != EMeshRenderAttributeFlags::None);
		}

		if (bUpdateSecondarySort)
		{
			Proxy->FastUpdateIndexBuffers(UpdatedSets);
		}

		// finish up, have to wait for background bounds recalculation here
		if (bPositions)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SimpleDynamicMeshComponent_FastROIUpdate_FinalPositionsUpdate);
			MarkRenderTransformDirty();
			UpdateBoundsCalc.Wait();
			UpdateBounds();
		}

		GetDynamicMesh()->PostRealtimeUpdate();
	}
}



/**
 * Compute the combined bounding-box of the Triangles array in parallel, by computing
 * partial boxes for subsets of this array, and then combining those boxes.
 * TODO: this should move to a pulbic utility function, and possibly the block-based ParallelFor
 * should be refactored out into something more general, as this pattern is useful in many places...
 */
static FAxisAlignedBox3d ParallelComputeROIBounds(const FDynamicMesh3& Mesh, const TArray<int32>& Triangles)
{
	FAxisAlignedBox3d FinalBounds = FAxisAlignedBox3d::Empty();
	FCriticalSection FinalBoundsLock;
	int32 N = Triangles.Num();
	constexpr int32 BlockSize = 4096;
	int32 Blocks = (N / BlockSize) + 1;
	ParallelFor(Blocks, [&](int bi)
	{
		FAxisAlignedBox3d BlockBounds = FAxisAlignedBox3d::Empty();
		for (int32 k = 0; k < BlockSize; ++k)
		{
			int32 i = bi * BlockSize + k;
			if (i < N)
			{
				int32 tid = Triangles[i];
				const FIndex3i& TriV = Mesh.GetTriangleRef(tid);
				BlockBounds.Contain(Mesh.GetVertexRef(TriV.A));
				BlockBounds.Contain(Mesh.GetVertexRef(TriV.B));
				BlockBounds.Contain(Mesh.GetVertexRef(TriV.C));
			}
		}
		FinalBoundsLock.Lock();
		FinalBounds.Contain(BlockBounds);
		FinalBoundsLock.Unlock();
	});
	return FinalBounds;
}



TFuture<bool> UDynamicMeshComponent::FastNotifyTriangleVerticesUpdated_TryPrecompute(
	const TArray<int32>& Triangles,
	TArray<int32>& UpdateSetsOut,
	FAxisAlignedBox3d& BoundsOut)
{
	if (!ensure(MeshObject) || (!!RenderMeshPostProcessor) || (GetCurrentSceneProxy() == nullptr) || (!Decomposition) || !AllowFastUpdate())
	{
		// is there a simpler way to do this? cannot seem to just make a TFuture<bool>...
		return Async(DynamicMeshComponentAsyncExecTarget, []() { return false; });
	}

	return Async(DynamicMeshComponentAsyncExecTarget, [this, &Triangles, &UpdateSetsOut, &BoundsOut]()
	{
		TFuture<void> ComputeBounds = Async(DynamicMeshComponentAsyncExecTarget, [this, &BoundsOut, &Triangles]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SimpleDynamicMeshComponent_FastVertexUpdatePrecomp_CalcBounds);
			BoundsOut = ParallelComputeROIBounds(*GetMesh(), Triangles);
		});

		TFuture<void> ComputeSets = Async(DynamicMeshComponentAsyncExecTarget, [this, &UpdateSetsOut, &Triangles]()
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(SimpleDynamicMeshComponent_FastVertexUpdatePrecomp_FindSets);
			int32 NumBuffers = Decomposition->Num();
			TArray<std::atomic<bool>> BufferFlags;
			BufferFlags.SetNum(NumBuffers);
			for (int32 k = 0; k < NumBuffers; ++k)
			{
				BufferFlags[k] = false;
			}
			ParallelFor(Triangles.Num(), [&](int32 k)
			{
				int32 SetID = Decomposition->GetGroupForTriangle(Triangles[k]);
				BufferFlags[SetID] = true;
			});
			UpdateSetsOut.Reset();
			for (int32 k = 0; k < NumBuffers; ++k)
			{
				if (BufferFlags[k])
				{
					UpdateSetsOut.Add(k);
				}
			}

		});

		ComputeSets.Wait();
		ComputeBounds.Wait();

		return true;
	});
}


void UDynamicMeshComponent::FastNotifyTriangleVerticesUpdated_ApplyPrecompute(
	const TArray<int32>& Triangles,
	EMeshRenderAttributeFlags UpdatedAttributes, 
	TFuture<bool>& Precompute, 
	const TArray<int32>& UpdateSets, 
	const FAxisAlignedBox3d& UpdateSetBounds)
{
	Precompute.Wait();

	bool bPrecomputeOK = Precompute.Get();
	if (bPrecomputeOK == false || GetCurrentSceneProxy() == nullptr || !AllowFastUpdate())
	{
		FastNotifyTriangleVerticesUpdated(Triangles, UpdatedAttributes);
		return;
	}

	FDynamicMeshSceneProxy* Proxy = GetCurrentSceneProxy();
	bool bPositions = (UpdatedAttributes & EMeshRenderAttributeFlags::Positions) != EMeshRenderAttributeFlags::None;
	bool bUpdateSecondarySort = (SecondaryTriFilterFunc) &&
		((UpdatedAttributes & EMeshRenderAttributeFlags::SecondaryIndexBuffers) != EMeshRenderAttributeFlags::None);

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SimpleDynamicMeshComponent_FastVertexUpdate_ApplyUpdate);
		Proxy->FastUpdateVertices(UpdateSets, bPositions,
			(UpdatedAttributes & EMeshRenderAttributeFlags::VertexNormals) != EMeshRenderAttributeFlags::None,
			(UpdatedAttributes & EMeshRenderAttributeFlags::VertexColors) != EMeshRenderAttributeFlags::None,
			(UpdatedAttributes & EMeshRenderAttributeFlags::VertexUVs) != EMeshRenderAttributeFlags::None);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SimpleDynamicMeshComponent_FastVertexUpdate_UpdateIndexBuffers);
		if (bUpdateSecondarySort)
		{
			Proxy->FastUpdateIndexBuffers(UpdateSets);
		}
	}

	if (bPositions)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(SimpleDynamicMeshComponent_FastVertexUpdate_FinalPositionsUpdate);
		MarkRenderTransformDirty();
		LocalBounds.Contain(UpdateSetBounds);
		UpdateBounds();
	}

	GetDynamicMesh()->PostRealtimeUpdate();
}





FPrimitiveSceneProxy* UDynamicMeshComponent::CreateSceneProxy()
{
	// if this is not always the case, we have made incorrect assumptions
	ensure(GetCurrentSceneProxy() == nullptr);

	FDynamicMeshSceneProxy* NewProxy = nullptr;
	if (MeshObject && GetMesh()->TriangleCount() > 0)
	{
		NewProxy = new FDynamicMeshSceneProxy(this);

		if (TriangleColorFunc)
		{
			NewProxy->MeshRenderBufferSetConverter.bUsePerTriangleColor = true;
			NewProxy->MeshRenderBufferSetConverter.PerTriangleColorFunc = [this](const FDynamicMesh3* MeshIn, int TriangleID) { return GetTriangleColor(MeshIn, TriangleID); };
		}
		else if ( GetColorOverrideMode() == EDynamicMeshComponentColorOverrideMode::Polygroups )
		{
			NewProxy->MeshRenderBufferSetConverter.bUsePerTriangleColor = true;
			NewProxy->MeshRenderBufferSetConverter.PerTriangleColorFunc = [this](const FDynamicMesh3* MeshIn, int TriangleID) { return GetGroupColor(MeshIn, TriangleID); };
		}

		if (HasVertexColorRemappingFunction())
		{
			NewProxy->MeshRenderBufferSetConverter.bApplyVertexColorRemapping = true;
			NewProxy->MeshRenderBufferSetConverter.VertexColorRemappingFunc = [this](FVector4f& Color) { RemapVertexColor(Color); };
		}

		if (SecondaryTriFilterFunc)
		{
			NewProxy->MeshRenderBufferSetConverter.bUseSecondaryTriBuffers = true;
			NewProxy->MeshRenderBufferSetConverter.SecondaryTriFilterFunc = [this](const FDynamicMesh3* MeshIn, int32 TriangleID)
			{ 
				return (SecondaryTriFilterFunc) ? SecondaryTriFilterFunc(MeshIn, TriangleID) : false;
			};
		}

		if (Decomposition)
		{
			NewProxy->InitializeFromDecomposition(Decomposition);
		}
		else
		{
			NewProxy->Initialize();
		}

		NewProxy->SetVerifyUsedMaterials(bProxyVerifyUsedMaterials);
	}

	bProxyValid = true;
	return NewProxy;
}



void UDynamicMeshComponent::NotifyMaterialSetUpdated()
{
	if (GetCurrentSceneProxy() != nullptr)
	{
		GetCurrentSceneProxy()->UpdatedReferencedMaterials();
	}
}







void UDynamicMeshComponent::SetTriangleColorFunction(
	TUniqueFunction<FColor(const FDynamicMesh3*, int)> TriangleColorFuncIn,
	EDynamicMeshComponentRenderUpdateMode UpdateMode)
{
	TriangleColorFunc = MoveTemp(TriangleColorFuncIn);

	if (UpdateMode == EDynamicMeshComponentRenderUpdateMode::FastUpdate)
	{
		FastNotifyColorsUpdated();
	}
	else if (UpdateMode == EDynamicMeshComponentRenderUpdateMode::FullUpdate)
	{
		NotifyMeshUpdated();
	}
}

void UDynamicMeshComponent::ClearTriangleColorFunction(EDynamicMeshComponentRenderUpdateMode UpdateMode)
{
	if (TriangleColorFunc)
	{
		TriangleColorFunc = nullptr;

		if (UpdateMode == EDynamicMeshComponentRenderUpdateMode::FastUpdate)
		{
			FastNotifyColorsUpdated();
		}
		else if (UpdateMode == EDynamicMeshComponentRenderUpdateMode::FullUpdate)
		{
			NotifyMeshUpdated();
		}
	}
}

bool UDynamicMeshComponent::HasTriangleColorFunction()
{
	return !!TriangleColorFunc;
}



void UDynamicMeshComponent::SetVertexColorRemappingFunction(
	TUniqueFunction<void(FVector4f&)> ColorMapFuncIn,
	EDynamicMeshComponentRenderUpdateMode UpdateMode)
{
	VertexColorMappingFunc = MoveTemp(ColorMapFuncIn);

	if (UpdateMode == EDynamicMeshComponentRenderUpdateMode::FastUpdate)
	{
		FastNotifyColorsUpdated();
	}
	else if (UpdateMode == EDynamicMeshComponentRenderUpdateMode::FullUpdate)
	{
		NotifyMeshUpdated();
	}
}

void UDynamicMeshComponent::ClearVertexColorRemappingFunction(EDynamicMeshComponentRenderUpdateMode UpdateMode)
{
	if (VertexColorMappingFunc)
	{
		VertexColorMappingFunc = nullptr;

		if (UpdateMode == EDynamicMeshComponentRenderUpdateMode::FastUpdate)
		{
			FastNotifyColorsUpdated();
		}
		else if (UpdateMode == EDynamicMeshComponentRenderUpdateMode::FullUpdate)
		{
			NotifyMeshUpdated();
		}
	}
}

bool UDynamicMeshComponent::HasVertexColorRemappingFunction()
{
	return !!VertexColorMappingFunc;
}


void UDynamicMeshComponent::RemapVertexColor(FVector4f& VertexColorInOut)
{
	if (VertexColorMappingFunc)
	{
		VertexColorMappingFunc(VertexColorInOut);
	}
}



void UDynamicMeshComponent::EnableSecondaryTriangleBuffers(TUniqueFunction<bool(const FDynamicMesh3*, int32)> SecondaryTriFilterFuncIn)
{
	SecondaryTriFilterFunc = MoveTemp(SecondaryTriFilterFuncIn);
	NotifyMeshUpdated();
}

void UDynamicMeshComponent::DisableSecondaryTriangleBuffers()
{
	SecondaryTriFilterFunc = nullptr;
	NotifyMeshUpdated();
}


void UDynamicMeshComponent::SetExternalDecomposition(TUniquePtr<FMeshRenderDecomposition> DecompositionIn)
{
	ensure(DecompositionIn->Num() > 0);
	Decomposition = MoveTemp(DecompositionIn);
	NotifyMeshUpdated();
}



FColor UDynamicMeshComponent::GetTriangleColor(const FDynamicMesh3* MeshIn, int TriangleID)
{
	if (TriangleColorFunc)
	{
		return TriangleColorFunc(MeshIn, TriangleID);
	}
	else
	{
		return (TriangleID % 2 == 0) ? FColor::Red : FColor::White;
	}
}


FColor UDynamicMeshComponent::GetGroupColor(const FDynamicMesh3* Mesh, int TriangleID) const
{
	int32 GroupID = Mesh->HasTriangleGroups() ? Mesh->GetTriangleGroup(TriangleID) : 0;
	return UE::Geometry::LinearColors::SelectFColor(GroupID);
}


FBoxSphereBounds UDynamicMeshComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	// can get a tighter box by calculating in world space, but we care more about performance
	FBox LocalBoundingBox = (FBox)LocalBounds;
	FBoxSphereBounds Ret(LocalBoundingBox.TransformBy(LocalToWorld));
	Ret.BoxExtent *= BoundsScale;
	Ret.SphereRadius *= BoundsScale;
	return Ret;
}




void UDynamicMeshComponent::SetInvalidateProxyOnChangeEnabled(bool bEnabled)
{
	bInvalidateProxyOnChange = bEnabled;
}


void UDynamicMeshComponent::ApplyChange(const FMeshVertexChange* Change, bool bRevert)
{
	// will fire UDynamicMesh::MeshChangedEvent, which will call OnMeshObjectChanged() below to invalidate proxy, fire change events, etc
	if (ensure(MeshObject))
	{
		MeshObject->ApplyChange(Change, bRevert);
	}
}

void UDynamicMeshComponent::ApplyChange(const FMeshChange* Change, bool bRevert)
{
	// will fire UDynamicMesh::MeshChangedEvent, which will call OnMeshObjectChanged() below to invalidate proxy, fire change events, etc
	if (ensure(MeshObject))
	{
		MeshObject->ApplyChange(Change, bRevert);
	}
}

void UDynamicMeshComponent::ApplyChange(const FMeshReplacementChange* Change, bool bRevert)
{
	// will fire UDynamicMesh::MeshChangedEvent, which will call OnMeshObjectChanged() below to invalidate proxy, fire change events, etc
	if (ensure(MeshObject))
	{
		MeshObject->ApplyChange(Change, bRevert);
	}
}

void UDynamicMeshComponent::OnMeshObjectChanged(UDynamicMesh* ChangedMeshObject, FDynamicMeshChangeInfo ChangeInfo)
{
	bool bIsFChange = (
		ChangeInfo.Type == EDynamicMeshChangeType::MeshChange
		|| ChangeInfo.Type == EDynamicMeshChangeType::MeshVertexChange
		|| ChangeInfo.Type == EDynamicMeshChangeType::MeshReplacementChange);

	if (bIsFChange)
	{
		if (bInvalidateProxyOnChange)
		{
			NotifyMeshUpdated();
		}

		OnMeshChanged.Broadcast();
		BroadcastMeshPropertyChangeEvent();

		if (ChangeInfo.Type == EDynamicMeshChangeType::MeshVertexChange)
		{
			OnMeshVerticesChanged.Broadcast(this, ChangeInfo.VertexChange, ChangeInfo.bIsRevertChange);
		}
		OnMeshRegionChanged.Broadcast(this, ChangeInfo.GetChange(), ChangeInfo.bIsRevertChange);
	}
	else
	{
		if (ChangeInfo.Type == EDynamicMeshChangeType::DeformationEdit)
		{
			// if ChangeType is a vertex deformation, we can do a fast-update of the vertex buffers
			// without fully rebuilding the SceneProxy
			EMeshRenderAttributeFlags UpdateFlags = UELocal::ConvertChangeFlagsToUpdateFlags(ChangeInfo.Flags);
			FastNotifyVertexAttributesUpdated(UpdateFlags);
		}
		else
		{
			NotifyMeshUpdated();
		}
		OnMeshChanged.Broadcast();
		BroadcastMeshPropertyChangeEvent();
	}

	InternalOnMeshUpdated();
}


void UDynamicMeshComponent::BroadcastMeshPropertyChangeEvent()
{
#if WITH_EDITOR
	if (FProperty* MeshProperty = UDynamicMeshComponent::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UDynamicMeshComponent, MeshObject)))
	{
		FPropertyChangedEvent MeshChangedEvent(MeshProperty);
		FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(this, MeshChangedEvent);
	}
#endif
}


void UDynamicMeshComponent::SetDynamicMesh(UDynamicMesh* NewMesh)
{
	if (ensure(NewMesh) == false)
	{
		return;
	}

	if (ensure(MeshObject))
	{
		MeshObject->OnMeshChanged().Remove(MeshObjectChangedHandle);
	}

	// set Outer of NewMesh to be this Component, ie transfer ownership. This is done via "renaming", which is
	// a bit odd, so the flags prevent some standard "renaming" behaviors from happening
	NewMesh->Rename( nullptr, this, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
	MeshObject = NewMesh;
	MeshObjectChangedHandle = MeshObject->OnMeshChanged().AddUObject(this, &UDynamicMeshComponent::OnMeshObjectChanged);

	NotifyMeshUpdated();
	OnMeshChanged.Broadcast();
	BroadcastMeshPropertyChangeEvent();

	InternalOnMeshUpdated();
}



void UDynamicMeshComponent::OnChildAttached(USceneComponent* ChildComponent)
{
	Super::OnChildAttached(ChildComponent);
	OnChildAttachmentModified.Broadcast(ChildComponent, true);
}
void UDynamicMeshComponent::OnChildDetached(USceneComponent* ChildComponent)
{
	Super::OnChildDetached(ChildComponent);
	OnChildAttachmentModified.Broadcast(ChildComponent, false);
}

void UDynamicMeshComponent::InternalOnMeshUpdated()
{
	// Rebuild physics data
	if (bDeferCollisionUpdates || bTransientDeferCollisionUpdates)
	{
		InvalidatePhysicsData();
	}
	else
	{
		RebuildPhysicsData();
	}
}

bool UDynamicMeshComponent::GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const
{
	ProcessMesh([&](const FDynamicMesh3& Mesh)
		{
			bool bCopyUVs = UPhysicsSettings::Get()->bSupportUVFromHitResults && Mesh.HasAttributes() && Mesh.Attributes()->NumUVLayers() > 0 && !bDisableMeshUVHitResults;
			if (bCopyUVs)
			{
				// conservative estimate
				OutTriMeshEstimates.VerticeCount = Mesh.TriangleCount() * 3;
			}
			else
			{
				OutTriMeshEstimates.VerticeCount = Mesh.VertexCount();
			}
		}
	);
	return true;
}

bool UDynamicMeshComponent::GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData)
{
	// this is something we currently assume, if you hit this ensure, we made a mistake
	ensure(bEnableComplexCollision);

	ProcessMesh([&](const FDynamicMesh3& Mesh)
	{
		// See if we should copy UVs
		const bool bCopyUVs = UPhysicsSettings::Get()->bSupportUVFromHitResults && Mesh.HasAttributes() && Mesh.Attributes()->NumUVLayers() > 0 && !bDisableMeshUVHitResults;
		if (bCopyUVs)
		{
			CollisionData->UVs.SetNum(Mesh.Attributes()->NumUVLayers());
		}
		const FDynamicMeshMaterialAttribute* MaterialAttrib = Mesh.HasAttributes() && Mesh.Attributes()->HasMaterialID() ? Mesh.Attributes()->GetMaterialID() : nullptr;

		TArray<int32> VertexMap; 
		const bool bIsSparseV = !Mesh.IsCompactV();

		// copy vertices
		if (!bCopyUVs)
		{
			if (bIsSparseV)
			{
				VertexMap.SetNum(Mesh.MaxVertexID());
			}
			CollisionData->Vertices.Reserve(Mesh.VertexCount());
			for (int32 vid : Mesh.VertexIndicesItr())
			{
				int32 Index = CollisionData->Vertices.Add((FVector3f)Mesh.GetVertex(vid));
				if (bIsSparseV)
				{
					VertexMap[vid] = Index;
				}
				else
				{
					check(vid == Index);
				}
			}
		}
		else
		{
			// map vertices per wedge
			VertexMap.SetNumZeroed(Mesh.MaxTriangleID() * 3);
			// temp array to store the UVs on a vertex (per triangle)
			TArray<FVector2D> VertUVs;
			const FDynamicMeshAttributeSet* Attribs = Mesh.Attributes();
			const int32 NumUVLayers = Attribs->NumUVLayers();
			for (int32 VID : Mesh.VertexIndicesItr())
			{
				FVector3f Pos = (FVector3f)Mesh.GetVertex(VID);
				int32 VertStart = CollisionData->Vertices.Num();
				Mesh.EnumerateVertexTriangles(VID, [&](int32 TID)
				{
					FIndex3i Tri = Mesh.GetTriangle(TID);
					int32 VSubIdx = Tri.IndexOf(VID);
					// Get the UVs on this wedge
					VertUVs.Reset(8);
					for (int32 UVIdx = 0; UVIdx < NumUVLayers; ++UVIdx)
					{
						const FDynamicMeshUVOverlay* Overlay = Attribs->GetUVLayer(UVIdx);
						FIndex3i UVTri = Overlay->GetTriangle(TID);
						int32 ElID = UVTri[VSubIdx];
						FVector2D UV(0, 0);
						if (ElID >= 0)
						{
							UV = (FVector2D)Overlay->GetElement(ElID);
						}
						VertUVs.Add(UV);
					}
					// Check if we've already added these UVs via an earlier wedge
					int32 OutputVIdx = INDEX_NONE;
					for (int32 VIdx = VertStart; VIdx < CollisionData->Vertices.Num(); ++VIdx)
					{
						bool bFound = true;
						for (int32 UVIdx = 0; UVIdx < NumUVLayers; ++UVIdx)
						{
							if (CollisionData->UVs[UVIdx][VIdx] != VertUVs[UVIdx])
							{
								bFound = false;
								break;
							}
						}
						if (bFound)
						{
							OutputVIdx = VIdx;
							break;
						}
					}
					// If not, add the vertex w/ the UVs
					if (OutputVIdx == INDEX_NONE)
					{
						OutputVIdx = CollisionData->Vertices.Add(Pos);
						for (int32 UVIdx = 0; UVIdx < NumUVLayers; ++UVIdx)
						{
							CollisionData->UVs[UVIdx].Add(VertUVs[UVIdx]);
						}
					}
					// Map the wedge to the output vertex
					VertexMap[TID * 3 + VSubIdx] = OutputVIdx;
				});
			}
		}

		// copy triangles
		CollisionData->Indices.Reserve(Mesh.TriangleCount());
		CollisionData->MaterialIndices.Reserve(Mesh.TriangleCount());
		for (int32 tid : Mesh.TriangleIndicesItr())
		{
			FIndex3i Tri = Mesh.GetTriangle(tid);
			FTriIndices Triangle;
			if (bCopyUVs)
			{
				// UVs need a wedge-based map
				Triangle.v0 = VertexMap[tid * 3 + 0];
				Triangle.v1 = VertexMap[tid * 3 + 1];
				Triangle.v2 = VertexMap[tid * 3 + 2];
			}
			else if (bIsSparseV)
			{
				Triangle.v0 = VertexMap[Tri.A];
				Triangle.v1 = VertexMap[Tri.B];
				Triangle.v2 = VertexMap[Tri.C];
			}
			else
			{ 
				Triangle.v0 = Tri.A;
				Triangle.v1 = Tri.B;
				Triangle.v2 = Tri.C;
			}

			// Filter out triangles which will cause physics system to emit degenerate-geometry warnings.
			// These checks reproduce tests in Chaos::CleanTrimesh
			const FVector3f& A = CollisionData->Vertices[Triangle.v0];
			const FVector3f& B = CollisionData->Vertices[Triangle.v1];
			const FVector3f& C = CollisionData->Vertices[Triangle.v2];
			if (A == B || A == C || B == C)
			{
				continue;
			}
			// anything that fails the first check should also fail this, but Chaos does both so doing the same here...
			const float SquaredArea = FVector3f::CrossProduct(A - B, A - C).SizeSquared();
			if (SquaredArea < UE_SMALL_NUMBER)
			{
				continue;
			}

			CollisionData->Indices.Add(Triangle);

			int32 MaterialID = MaterialAttrib ? MaterialAttrib->GetValue(tid) : 0;
			CollisionData->MaterialIndices.Add(MaterialID);
		}

		CollisionData->bFlipNormals = true;
		CollisionData->bDeformableMesh = true;
		CollisionData->bFastCook = true;
	});

	return true;
}

bool UDynamicMeshComponent::ContainsPhysicsTriMeshData(bool InUseAllTriData) const
{
	if (bEnableComplexCollision && (MeshObject != nullptr))
	{
		int32 TriangleCount = MeshObject->GetTriangleCount();

		// if the triangle count is too large, skip building complex collision
		int32 MaxComplexCollisionTriCount = CVarDynamicMeshComponent_MaxComplexCollisionTriCount.GetValueOnAnyThread();
		if (MaxComplexCollisionTriCount >= 0 && TriangleCount > MaxComplexCollisionTriCount)
		{
			static bool bHavePrintedWarningMessage = false;
			if (!bHavePrintedWarningMessage)
			{
				UE_LOGF(LogGeometry, Warning, "Ignoring attempt to build Complex Collision for a DynamicMeshComponent with triangle count larger than %d. Increase the geometry.DynamicMesh.MaxComplexCollisionTriCount value if you are certain you want to build Complex Collision for very large meshes.", MaxComplexCollisionTriCount);
				bHavePrintedWarningMessage = true;
			}
			return false;
		}
		if (TriangleCount > 0)
		{
			return true;
		}
	}
	return false;
}

bool UDynamicMeshComponent::WantsNegXTriMesh()
{
	return true;
}

UBodySetup* UDynamicMeshComponent::CreateBodySetupHelper()
{
	UBodySetup* NewBodySetup = nullptr;
	{
		FGCScopeGuard Scope;

		// Below flags are copied from UProceduralMeshComponent::CreateBodySetupHelper(). Without these flags, DynamicMeshComponents inside
		// a DynamicMeshActor BP will result on a GLEO error after loading and modifying a saved Level (but *not* on the initial save)
		// The UBodySetup in a template needs to be public since the property is Instanced and thus is the archetype of the instance meaning there is a direct reference
		NewBodySetup = NewObject<UBodySetup>(this, NAME_None, (IsTemplate() ? RF_Public | RF_ArchetypeObject : RF_NoFlags));
	}
	NewBodySetup->BodySetupGuid = FGuid::NewGuid();

	NewBodySetup->bGenerateMirroredCollision = false;
	NewBodySetup->CollisionTraceFlag = this->CollisionType;

	NewBodySetup->DefaultInstance.SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);
	NewBodySetup->bSupportUVsAndFaceRemap = false; /* bSupportPhysicalMaterialMasks; */

	return NewBodySetup;
}

UBodySetup* UDynamicMeshComponent::GetBodySetup()
{
	if (MeshBodySetup == nullptr)
	{
		UBodySetup* NewBodySetup = CreateBodySetupHelper();
		
		SetBodySetup(NewBodySetup);
	}

	return MeshBodySetup;
}

void UDynamicMeshComponent::SetBodySetup(UBodySetup* NewSetup)
{
	if (ensure(NewSetup))
	{
		MeshBodySetup = NewSetup;
	}
}

void UDynamicMeshComponent::SetSimpleCollisionShapes(const struct FKAggregateGeom& AggGeomIn, bool bUpdateCollision)
{
	AggGeom = AggGeomIn;
	if (bUpdateCollision)
	{
		UpdateCollision(false);
	}
}

void UDynamicMeshComponent::ClearSimpleCollisionShapes(bool bUpdateCollision)
{
	AggGeom.EmptyElements();
	if (bUpdateCollision)
	{
		UpdateCollision(false);
	}
}

void UDynamicMeshComponent::InvalidatePhysicsData()
{
	if (GetBodySetup())
	{
		GetBodySetup()->InvalidatePhysicsData();
		bCollisionUpdatePending = true;
	}
}

void UDynamicMeshComponent::RebuildPhysicsData()
{
	UWorld* World = GetWorld();
	const bool bUseAsyncCook = bUseAsyncCooking
		// It's uncertain whether these checks are actually needed. UProceduralMeshComponent checked for a game,
		//  not editor, world. It's possible that at the time the editor was not ticked in a way that caused
		//  physics builds to complete.
		// We experimentally allow asynchronous builds in editor via a cvar, to support cases where in-editor dynamic meshes
		//  may otherwise hitch during editing.
		&& World && (World->IsGameWorld() || 
			(World->IsEditorWorld() && CVarDynamicMeshComponent_AllowCollisionAsyncBuildInEditor.GetValueOnAnyThread())
			);

	UBodySetup* BodySetup = nullptr;
	if (bUseAsyncCook)
	{
		// Abort all previous ones still standing
		for (UBodySetup* OldBody : AsyncBodySetupQueue)
		{
			if (OldBody)
			{
				OldBody->AbortPhysicsMeshAsyncCreation();
			}
		}

		BodySetup = CreateBodySetupHelper();
		if (BodySetup)
		{
			AsyncBodySetupQueue.Add(BodySetup);
		}
	}
	else
	{
		AsyncBodySetupQueue.Empty();	// If for some reason we modified the async at runtime, just clear any pending async body setups
		BodySetup = GetBodySetup();
	}

	if (!BodySetup)
	{
		return;
	}

	BodySetup->CollisionTraceFlag = this->CollisionType;
	// Note: Directly assigning AggGeom wouldn't do some important-looking cleanup (clearing pointers on convex elements)
	//  so we RemoveSimpleCollision then AddCollisionFrom instead
	BodySetup->RemoveSimpleCollision();
	BodySetup->AddCollisionFrom(this->AggGeom);

	if (bUseAsyncCook)
	{
		BodySetup->CreatePhysicsMeshesAsync(FOnAsyncPhysicsCookFinished::CreateUObject(this, &UDynamicMeshComponent::FinishPhysicsAsyncCook, BodySetup));
	}
	else
	{
		// New GUID as collision has changed
		BodySetup->BodySetupGuid = FGuid::NewGuid();
		// Also we want cooked data for this
		BodySetup->bHasCookedCollisionData = true;
		BodySetup->InvalidatePhysicsData();
		BodySetup->CreatePhysicsMeshes();
		RecreatePhysicsState();

		bCollisionUpdatePending = false;
	}

	if (FDynamicMeshSceneProxy* Proxy = GetCurrentSceneProxy())
	{
		Proxy->SetCollisionData();
	}
}

void UDynamicMeshComponent::FinishPhysicsAsyncCook(bool bSuccess, UBodySetup* FinishedBodySetup)
{
	TArray<UBodySetup*> NewQueue;
	NewQueue.Reserve(AsyncBodySetupQueue.Num());

	int32 FoundIdx;
	if (AsyncBodySetupQueue.Find(FinishedBodySetup, FoundIdx))
	{
		// Note: currently no-cook-needed is reported identically to cook failed.
		// Checking AggGeom.ConvexElems & ContainsPhysicsTriMeshData here is a hack to distinguish the no-cook-needed case
		// These checks mirror those from UBodySetup::GetCookInfo.
		// TODO: remove this hack to distinguish the no-cook-needed case when/if that is no longer identical to the cook failed case
		const ECollisionTraceFlag BodyCollisionType = FinishedBodySetup->GetCollisionTraceFlag();
		const bool bEmptySimpleCollision = FinishedBodySetup->AggGeom.ConvexElems.Num() == 0;
		const bool bEmptyComplexCollision = !ContainsPhysicsTriMeshData(false);
		const bool bNoCookNeeded =
			(BodyCollisionType == CTF_UseSimpleAsComplex && bEmptySimpleCollision) ||
			(BodyCollisionType == CTF_UseComplexAsSimple && bEmptyComplexCollision) ||
			(BodyCollisionType == CTF_UseSimpleAndComplex && bEmptySimpleCollision && bEmptyComplexCollision);
		if (bSuccess || bNoCookNeeded)
		{
			// The new body was found in the array meaning it's newer, so use it
			MeshBodySetup = FinishedBodySetup;
			RecreatePhysicsState();

			// remove any async body setups that were requested before this one
			for (int32 AsyncIdx = FoundIdx + 1; AsyncIdx < AsyncBodySetupQueue.Num(); ++AsyncIdx)
			{
				NewQueue.Add(AsyncBodySetupQueue[AsyncIdx]);
			}

			AsyncBodySetupQueue = NewQueue;
		}
		else
		{
			AsyncBodySetupQueue.RemoveAt(FoundIdx);
		}
	}
}

void UDynamicMeshComponent::UpdateCollision(bool bOnlyIfPending)
{
	if (bOnlyIfPending == false || bCollisionUpdatePending)
	{
		RebuildPhysicsData();
	}
}

void UDynamicMeshComponent::BeginDestroy()
{
	Super::BeginDestroy();

	AggGeom.FreeRenderInfo();
}

void UDynamicMeshComponent::EnableComplexAsSimpleCollision()
{
	SetComplexAsSimpleCollisionEnabled(true, true);
}

void UDynamicMeshComponent::SetComplexAsSimpleCollisionEnabled(bool bEnabled, bool bImmediateUpdate)
{
	bool bModified = false;
	if (bEnabled)
	{
		if (bEnableComplexCollision == false)
		{
			bEnableComplexCollision = true;
			bModified = true;
		}
		if (CollisionType != ECollisionTraceFlag::CTF_UseComplexAsSimple)
		{
			CollisionType = ECollisionTraceFlag::CTF_UseComplexAsSimple;
			bModified = true;
		}
	}
	else
	{
		if (bEnableComplexCollision == true)
		{
			bEnableComplexCollision = false;
			bModified = true;
		}
		if (CollisionType == ECollisionTraceFlag::CTF_UseComplexAsSimple)
		{
			CollisionType = ECollisionTraceFlag::CTF_UseDefault;
			bModified = true;
		}
	}
	if (bModified)
	{
		InvalidatePhysicsData();
	}
	if (bImmediateUpdate)
	{
		UpdateCollision(true);
	}
}


void UDynamicMeshComponent::SetDeferredCollisionUpdatesEnabled(bool bEnabled, bool bImmediateUpdate)
{
	if (bDeferCollisionUpdates != bEnabled)
	{
		bDeferCollisionUpdates = bEnabled;
		if (bEnabled == false && bImmediateUpdate)
		{
			UpdateCollision(true);
		}
	}
}

void UDynamicMeshComponent::SetTransientDeferCollisionUpdates(bool bEnabled)
{
	bTransientDeferCollisionUpdates = bEnabled;
}

void UDynamicMeshComponent::SetSceneProxyVerifyUsedMaterials(bool bState)
{
	bProxyVerifyUsedMaterials = bState;
	if (FDynamicMeshSceneProxy* Proxy = GetCurrentSceneProxy())
	{
		Proxy->SetVerifyUsedMaterials(bState);
	}
}


