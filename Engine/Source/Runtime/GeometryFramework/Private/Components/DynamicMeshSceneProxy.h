// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "Components/DynamicMeshComponent.h"
#include "Components/BaseDynamicMeshSceneProxy.h"
#include "DynamicMesh/MeshTangents.h"
#include "Async/ParallelFor.h"
#include "Materials/Material.h"
#include "MaterialDomain.h"
#include "MaterialShared.h"
#include "SceneInterface.h"

/**
 * Scene Proxy for UDynamicMeshComponent.
 * 
 * Based on FProceduralMeshSceneProxy but simplified in various ways.
 * 
 * Supports wireframe-on-shaded rendering.
 * 
 */
class FDynamicMeshSceneProxy final : public FBaseDynamicMeshSceneProxy
{
	using FIndex2i = UE::Geometry::FIndex2i;
	using FIndex3i = UE::Geometry::FIndex3i;
private:
	FMaterialRelevance MaterialRelevance;

	// note: FBaseDynamicMeshSceneProxy owns and will destroy these
	TArray<FMeshRenderBufferSet*> RenderBufferSets;

	// if true, we store entire mesh in single RenderBuffers and we can do some optimizations
	bool bIsSingleBuffer = false;

public:
	/** Component that created this proxy (is there a way to look this up?) */
	UDynamicMeshComponent* ParentComponent;


	FDynamicMeshSceneProxy(UDynamicMeshComponent* Component)
		: FBaseDynamicMeshSceneProxy(Component)
		, MaterialRelevance(Component->GetMaterialRelevance(GetScene().GetShaderPlatform()))
	{
		ParentComponent = Component;
	}


	virtual void GetActiveRenderBufferSets(TArray<FMeshRenderBufferSet*>& Buffers) const override
	{
		Buffers = RenderBufferSets;
	}



	void Initialize()
	{
		// allocate buffer sets based on materials
		ensure(RenderBufferSets.Num() == 0);
		int32 NumMaterials = GetNumMaterials();
		if (NumMaterials == 0)
		{
			RenderBufferSets.SetNum(1);
			RenderBufferSets[0] = AllocateNewRenderBufferSet();
			RenderBufferSets[0]->Material = UMaterial::GetDefaultMaterial(MD_Surface);
		}
		else
		{
			RenderBufferSets.SetNum(NumMaterials);
			for (int32 k = 0; k < NumMaterials; ++k)
			{
				RenderBufferSets[k] = AllocateNewRenderBufferSet();
				RenderBufferSets[k]->Material = GetMaterial(k);
			}
		}


		FDynamicMesh3* Mesh = ParentComponent->GetRenderMesh();
		if (Mesh->HasAttributes() && Mesh->Attributes()->HasMaterialID() && NumMaterials > 1)
		{
			bIsSingleBuffer = false;
			InitializeByMaterial(RenderBufferSets);
		}
		else
		{
			bIsSingleBuffer = true;
			InitializeSingleBufferSet(RenderBufferSets[0]);
		}

		// set (or clear) simple Lumen cards via the bounding box
		// TODO: Implement a better method to set up the lumen cards
		UpdateLumenCardsFromBounds();
	}



	TUniqueFunction<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> MakeTangentsFunc(bool bSkipAutoCompute = false)
	{
		EDynamicMeshComponentTangentsMode TangentsType = ParentComponent->GetTangentsType();
		if (TangentsType == EDynamicMeshComponentTangentsMode::ExternallyProvided)
		{
			UE::Geometry::FDynamicMesh3* RenderMesh = ParentComponent->GetRenderMesh();
			// If the RenderMesh has tangents, use them. Otherwise we fall back to the orthogonal basis, below.
			if (RenderMesh && RenderMesh->HasAttributes() && RenderMesh->Attributes()->HasTangentSpace())
			{
				UE::Geometry::FDynamicMeshTangents Tangents(RenderMesh);
				return [Tangents](int VertexID, int TriangleID, int TriVtxIdx, const FVector3f& Normal, FVector3f& TangentX, FVector3f& TangentY) -> void
				{
					Tangents.GetTangentVectors(TriangleID, TriVtxIdx, Normal, TangentX, TangentY);
				};
			}
		}
		else if (TangentsType == EDynamicMeshComponentTangentsMode::AutoCalculated && bSkipAutoCompute == false)
		{
			const UE::Geometry::FMeshTangentsf* Tangents = ParentComponent->GetAutoCalculatedTangents();
			if (Tangents != nullptr )
			{
				return [Tangents](int VertexID, int TriangleID, int TriVtxIdx, const FVector3f& Normal, FVector3f& TangentX, FVector3f& TangentY) -> void
				{
					Tangents->GetTriangleVertexTangentVectors(TriangleID, TriVtxIdx, TangentX, TangentY);
				};
			}
		}

		// fallback to orthogonal basis
		return [](int VertexID, int TriangleID, int TriVtxIdx, const FVector3f& Normal, FVector3f& TangentX, FVector3f& TangentY) -> void
		{
			UE::Geometry::VectorUtil::MakePerpVectors(Normal, TangentX, TangentY);
		};
	}


	/**
	 * Initialize multiple buffers for the mesh based on the given Decomposition
	 */
	void InitializeFromDecomposition(TUniquePtr<FMeshRenderDecomposition>& Decomposition)
	{
		ensure(RenderBufferSets.Num() == 0);
		int32 NumSets = Decomposition->Num();
		RenderBufferSets.SetNum(NumSets);
		for (int32 k = 0; k < NumSets; ++k)
		{
			RenderBufferSets[k] = AllocateNewRenderBufferSet();
			RenderBufferSets[k]->Material = Decomposition->GetGroup(k).Material;
			if (RenderBufferSets[k]->Material == nullptr)
			{
				RenderBufferSets[k]->Material = UMaterial::GetDefaultMaterial(MD_Surface);
			}
		}

		bIsSingleBuffer = false;

		const FDynamicMesh3* Mesh = ParentComponent->GetRenderMesh();
		const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();

		// find suitable overlays
		TArray<const FDynamicMeshUVOverlay*, TInlineAllocator<8>> UVOverlays;
		UVOverlays.SetNum(Attributes->NumUVLayers());
		for (int32 k = 0; k < UVOverlays.Num(); ++k)
		{
			UVOverlays[k] = Attributes->GetUVLayer(k);
		}
		const FDynamicMeshNormalOverlay* NormalOverlay = Attributes->PrimaryNormals();
		const FDynamicMeshColorOverlay* ColorOverlay = Attributes->PrimaryColors();
		TUniqueFunction<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc = MakeTangentsFunc();

		// init renderbuffers for each set
		ParallelFor(NumSets, [&](int32 SetIndex)
		{
			const FMeshRenderDecomposition::FGroup& Group = Decomposition->GetGroup(SetIndex);
			const TArray<int32>& Triangles = Group.Triangles;
			if (Triangles.Num() > 0)
			{
				FMeshRenderBufferSet* RenderBuffers = RenderBufferSets[SetIndex];
				RenderBuffers->Triangles = Triangles;
				InitializeBuffersFromOverlays(RenderBuffers, Mesh,
					Triangles.Num(), Triangles,
					UVOverlays, NormalOverlay, ColorOverlay, TangentsFunc);

				ENQUEUE_RENDER_COMMAND(FDynamicMeshSceneProxyInitializeFromDecomposition)(
					[RenderBuffers](FRHICommandListImmediate& RHICmdList)
				{
					RenderBuffers->Upload();
				});
			}
		});
	}








	/**
	 * Initialize a single set of mesh buffers for the entire mesh
	 */
	void InitializeSingleBufferSet(FMeshRenderBufferSet* RenderBuffers)
	{
		const FDynamicMesh3* Mesh = ParentComponent->GetRenderMesh();

		// find suitable overlays
		TArray<const FDynamicMeshUVOverlay*, TInlineAllocator<8>> UVOverlays;
		const FDynamicMeshNormalOverlay* NormalOverlay = nullptr;
		const FDynamicMeshColorOverlay* ColorOverlay = nullptr;
		if (Mesh->HasAttributes())
		{
			const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();
			NormalOverlay = Attributes->PrimaryNormals();
			UVOverlays.SetNum(Attributes->NumUVLayers());
			for (int32 k = 0; k < UVOverlays.Num(); ++k)
			{
				UVOverlays[k] = Attributes->GetUVLayer(k);
			}
			ColorOverlay = Attributes->PrimaryColors();
		}
		TUniqueFunction<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc = MakeTangentsFunc();

		const bool bTrackTriangles = false;
		const bool bParallel = true;
		InitializeBuffersFromOverlays(RenderBuffers, Mesh,
			Mesh->TriangleCount(), Mesh->TriangleIndicesItr(),
			UVOverlays, NormalOverlay, ColorOverlay, TangentsFunc,
			bTrackTriangles, bParallel);

		ENQUEUE_RENDER_COMMAND(FDynamicMeshSceneProxyInitializeSingle)(
			[RenderBuffers](FRHICommandListImmediate& RHICmdList)
		{
			RenderBuffers->Upload();
		});
	}



	/**
	 * Initialize the mesh buffers, one per material
	 */
	void InitializeByMaterial(TArray<FMeshRenderBufferSet*>& BufferSets)
	{
		const FDynamicMesh3* Mesh = ParentComponent->GetRenderMesh();
		if (ensure(Mesh->HasAttributes() && Mesh->Attributes()->HasMaterialID()) == false)
		{
			return;
		}

		const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();

		// find suitable overlays
		const FDynamicMeshMaterialAttribute* MaterialID = Attributes->GetMaterialID();
		const FDynamicMeshNormalOverlay* NormalOverlay = Mesh->Attributes()->PrimaryNormals();
		const FDynamicMeshColorOverlay* ColorOverlay = Mesh->Attributes()->PrimaryColors();

		TArray<const FDynamicMeshUVOverlay*, TInlineAllocator<8>> UVOverlays;
		UVOverlays.SetNum(Attributes->NumUVLayers());
		for (int32 k = 0; k < UVOverlays.Num(); ++k)
		{
			UVOverlays[k] = Attributes->GetUVLayer(k);
		}

		TUniqueFunction<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc = MakeTangentsFunc();

		// count number of triangles for each material (could do in parallel?)
		int NumMaterials = BufferSets.Num();
		TArray<FThreadSafeCounter> Counts;
		Counts.SetNum(NumMaterials);
		for (int k = 0; k < NumMaterials; ++k)
		{
			Counts[k].Reset();
		}
		ParallelFor(Mesh->MaxTriangleID(), [&](int tid)
		{
			if (!Mesh->IsTriangle(tid))
			{
				return;
			}
			int MatIdx;
			MaterialID->GetValue(tid, &MatIdx);
			if (MatIdx >= 0 && MatIdx < NumMaterials)
			{
				Counts[MatIdx].Increment();
			}
		});

		// find max count
		int32 MaxCount = 0;
		for (FThreadSafeCounter& Count : Counts)
		{
			MaxCount = FMath::Max(Count.GetValue(), MaxCount);
		}

		// init renderbuffers for each material
		// could do this in parallel but then we need to allocate separate triangle arrays...is it worth it?
		TArray<int> Triangles;
		Triangles.Reserve(MaxCount);
		for (int k = 0; k < NumMaterials; ++k)
		{
			if (Counts[k].GetValue() > 0)
			{
				FMeshRenderBufferSet* RenderBuffers = BufferSets[k];

				Triangles.Reset();
				for (int tid : Mesh->TriangleIndicesItr())
				{
					int MatIdx;
					MaterialID->GetValue(tid, &MatIdx);
					if (MatIdx == k)
					{
						Triangles.Add(tid);
					}
				}

				const bool bTrackTriangles = false;
				const bool bParallel = true;
				InitializeBuffersFromOverlays(RenderBuffers, Mesh,
					Triangles.Num(), Triangles,
					UVOverlays, NormalOverlay, ColorOverlay, TangentsFunc,
					bTrackTriangles, bParallel);

				RenderBuffers->Triangles = Triangles;

				ENQUEUE_RENDER_COMMAND(FDynamicMeshSceneProxyInitializeByMaterial)(
					[RenderBuffers](FRHICommandListImmediate& RHICmdList)
				{
					RenderBuffers->Upload();
				});
			}
		}
	}


	bool RenderMeshLayoutMatchesRenderBuffers() const
	{
		const FDynamicMesh3* Mesh = ParentComponent->GetRenderMesh();

		auto CheckBufferSet = [](const FDynamicMesh3* Mesh, const FMeshRenderBufferSet* BufferSet, int NumTriangles) -> bool
		{
			if (BufferSet->Triangles)
			{
				for (int TriangleID : BufferSet->Triangles.GetValue())
				{
					if (!Mesh->IsTriangle(TriangleID))
					{
						return false;
					}
				}
			}

			int NumVertices = NumTriangles * 3;
			if (BufferSet->PositionVertexBuffer.GetNumVertices() != NumVertices ||
				BufferSet->StaticMeshVertexBuffer.GetNumVertices() != NumVertices ||
				BufferSet->ColorVertexBuffer.GetNumVertices() != NumVertices)
			{
				return false;
			}

			return true;
		};

		if (bIsSingleBuffer)
		{
			if (ensure(RenderBufferSets.Num() == 1))
			{
				FMeshRenderBufferSet* BufferSet = RenderBufferSets[0];
				FScopeLock BuffersLock(&BufferSet->BuffersLock);
				if (BufferSet->TriangleCount != Mesh->TriangleCount())
				{
					return false;
				}

				int NumTriangles = Mesh->TriangleCount();

				if (!CheckBufferSet(Mesh, BufferSet, NumTriangles))
				{
					return false;
				}
			}
		}
		else
		{
			for (FMeshRenderBufferSet* BufferSet : RenderBufferSets)
			{
				FScopeLock BuffersLock(&BufferSet->BuffersLock);
				if (ensure(BufferSet->Triangles))
				{
					int NumTriangles = BufferSet->Triangles->Num();

					if (!CheckBufferSet(Mesh, BufferSet, NumTriangles))
					{
						return false;
					}
				}
			}
		}

		return true;
	}



	/**
	 * Update the vertex position/normal/color buffers
	 */
	void FastUpdateVertices(bool bPositions, bool bNormals, bool bColors, bool bUVs)
	{
		// This needs to be rewritten for split-by-material buffers.
		// Could store triangle set with each buffer, and then rebuild vtx buffer(s) as needed?

		FDynamicMesh3* Mesh = ParentComponent->GetRenderMesh();

		// find suitable overlays and attributes
		FDynamicMeshNormalOverlay* NormalOverlay = nullptr;
		if (bNormals && ensure(Mesh->HasAttributes()) )
		{
			NormalOverlay = Mesh->Attributes()->PrimaryNormals();
		}
		TArray<const FDynamicMeshUVOverlay*, TInlineAllocator<8>> UVOverlays;
		if (bUVs && ensure(Mesh->HasAttributes()) )
		{
			const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();
			UVOverlays.SetNum(Attributes->NumUVLayers());
			for (int32 i = 0; i < UVOverlays.Num(); ++i)
			{
				UVOverlays[i] = Attributes->GetUVLayer(i);
			}
		}
		FDynamicMeshColorOverlay* ColorOverlay = nullptr;
		if (bColors && ensure(Mesh->HasAttributes()) )
		{
			ColorOverlay = Mesh->Attributes()->PrimaryColors();
		}

		// Currently deferring tangents auto-computation if doing fast vertex update (maybe needs to be exposed?)
		TUniqueFunction<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc = MakeTangentsFunc(true);

		if (bIsSingleBuffer)
		{
			if (ensure(RenderBufferSets.Num() == 1))
			{
				FMeshRenderBufferSet* Buffers = RenderBufferSets[0];
				FScopeLock BuffersLock(&Buffers->BuffersLock);
				if (bPositions || bNormals || bColors)
				{
					UpdateVertexBuffersFromOverlays(Buffers, Mesh,
						Mesh->TriangleCount(), Mesh->TriangleIndicesItr(),
						NormalOverlay, ColorOverlay, TangentsFunc,
						bPositions, bNormals, bColors);
				}
				if (bUVs)
				{
					UpdateVertexUVBufferFromOverlays(Buffers, Mesh,
						Mesh->TriangleCount(), Mesh->TriangleIndicesItr(), UVOverlays);
				}


				ENQUEUE_RENDER_COMMAND(FDynamicMeshSceneProxyFastUpdateVertices)(
					[Buffers, bPositions, bNormals, bColors, bUVs](FRHICommandListImmediate& RHICmdList)
				{
					Buffers->UploadVertexUpdate(bPositions, bNormals || bUVs, bColors);
				});
			}
		}
		else
		{
			ParallelFor(RenderBufferSets.Num(), [&](int i)
			{
				FMeshRenderBufferSet* Buffers = RenderBufferSets[i];
				FScopeLock BuffersLock(&Buffers->BuffersLock);
				if (Buffers->TriangleCount == 0)
				{
					return;
				}
				if (ensure(Buffers->Triangles.IsSet()))
				{
					if (bPositions || bNormals || bColors)
					{
						UpdateVertexBuffersFromOverlays(Buffers, Mesh,
							Buffers->Triangles->Num(), Buffers->Triangles.GetValue(),
							NormalOverlay, ColorOverlay, TangentsFunc,
							bPositions, bNormals, bColors);
					}
					if (bUVs)
					{
						UpdateVertexUVBufferFromOverlays(Buffers, Mesh,
							Buffers->Triangles->Num(), Buffers->Triangles.GetValue(), UVOverlays);
					}

					ENQUEUE_RENDER_COMMAND(FDynamicMeshSceneProxyFastUpdateVertices)(
						[Buffers, bPositions, bNormals, bColors, bUVs](FRHICommandListImmediate& RHICmdList)
					{
						Buffers->UploadVertexUpdate(bPositions, bNormals || bUVs, bColors);
					});
				}
			});
		}
	}



	/**
	 * Update the vertex position/normal/color buffers
	 */
	void FastUpdateVertices(const TArray<int32>& WhichBuffers, bool bPositions, bool bNormals, bool bColors, bool bUVs)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(DynamicMeshProxy_FastUpdateVertices);

		// skip if we have no updates
		if (bPositions == false && bNormals == false && bColors == false && bUVs == false)
		{
			return;
		}

		// This needs to be rewritten for split-by-material buffers.
		// Could store triangle set with each buffer, and then rebuild vtx buffer(s) as needed?

		const FDynamicMesh3* Mesh = ParentComponent->GetRenderMesh();

		// find suitable overlays
		const FDynamicMeshNormalOverlay* NormalOverlay = nullptr;
		if (bNormals && ensure(Mesh->HasAttributes()))
		{
			NormalOverlay = Mesh->Attributes()->PrimaryNormals();
		}
		TArray<const FDynamicMeshUVOverlay*> UVOverlays;
		if (bUVs && ensure(Mesh->HasAttributes()))
		{
			const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();
			UVOverlays.SetNum(Attributes->NumUVLayers());
			for (int32 i = 0; i < UVOverlays.Num(); ++i)
			{
				UVOverlays[i] = Attributes->GetUVLayer(i);
			}
		}
		const FDynamicMeshColorOverlay* ColorOverlay = nullptr;
		if (bColors && ensure(Mesh->HasAttributes()))
		{
			ColorOverlay = Mesh->Attributes()->PrimaryColors();
		}

		// Currently deferring tangents auto-computation if doing fast vertex update (maybe needs to be exposed?)
		TUniqueFunction<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc = MakeTangentsFunc(true);

		ParallelFor(WhichBuffers.Num(), [&](int idx)
		{
			int32 BufferIndex = WhichBuffers[idx];
			if ( RenderBufferSets.IsValidIndex(BufferIndex) == false)
			{
				return;
			}
			FMeshRenderBufferSet* Buffers = RenderBufferSets[BufferIndex];
			FScopeLock BuffersLock(&Buffers->BuffersLock);
			if (Buffers->TriangleCount == 0)
			{
				return;
			}
			if (ensure(Buffers->Triangles.IsSet()))
			{
				if (bPositions || bNormals || bColors)
				{
					UpdateVertexBuffersFromOverlays(Buffers, Mesh,
						Buffers->Triangles->Num(), Buffers->Triangles.GetValue(),
						NormalOverlay, ColorOverlay, TangentsFunc,
						bPositions, bNormals, bColors);
				}
				if (bUVs)
				{
					UpdateVertexUVBufferFromOverlays(Buffers, Mesh,
						Buffers->Triangles->Num(), Buffers->Triangles.GetValue(), UVOverlays);
				}

				ENQUEUE_RENDER_COMMAND(FDynamicMeshSceneProxyFastUpdateVerticesBufferList)(
					[Buffers, bPositions, bNormals, bColors, bUVs](FRHICommandListImmediate& RHICmdList)
				{
					Buffers->TransferVertexUpdateToGPU(RHICmdList, bPositions, bNormals, bUVs, bColors);
				});
			}
		});
	}




	/**
	 * Update index buffers inside each RenderBuffer set
	 */
	void FastUpdateAllIndexBuffers()
	{
		FDynamicMesh3* Mesh = ParentComponent->GetRenderMesh();

		// have to wait for all outstanding rendering to finish because the index buffers we are about to edit might be in-use
		FlushRenderingCommands();

		ParallelFor(RenderBufferSets.Num(), [this, &Mesh](int i)
		{
			FMeshRenderBufferSet* Buffers = RenderBufferSets[i];

			FScopeLock BuffersLock(&Buffers->BuffersLock);

			if (Buffers->TriangleCount > 0)
			{
				RecomputeRenderBufferTriangleIndexSets(Buffers, Mesh);
			}

			ENQUEUE_RENDER_COMMAND(FDynamicMeshSceneProxyFastUpdateAllIndexBuffers)(
				[Buffers](FRHICommandListImmediate& RHICmdList)
			{
				Buffers->UploadIndexBufferUpdate();
			});
			
		});
	}



	/**
	 * Update index buffers inside each RenderBuffer set
	 */
	void FastUpdateIndexBuffers(const TArray<int32>& WhichBuffers)
	{
		FDynamicMesh3* Mesh = ParentComponent->GetRenderMesh();

		// have to wait for all outstanding rendering to finish because the index buffers we are about to edit might be in-use
		FlushRenderingCommands();

		ParallelFor(WhichBuffers.Num(), [this, &WhichBuffers, &Mesh](int i)
		{
			int32 BufferIndex = WhichBuffers[i];
			if (RenderBufferSets.IsValidIndex(BufferIndex) == false)
			{
				return;
			}

			FMeshRenderBufferSet* Buffers = RenderBufferSets[BufferIndex];
			FScopeLock BuffersLock(&Buffers->BuffersLock);
			if (Buffers->TriangleCount > 0)
			{
				RecomputeRenderBufferTriangleIndexSets(Buffers, Mesh);
			}

			ENQUEUE_RENDER_COMMAND(FDynamicMeshSceneProxyFastUpdateSomeIndexBuffers)(
				[Buffers](FRHICommandListImmediate& RHICmdList)
			{
				Buffers->UploadIndexBufferUpdate();
			});
		});
	}



public:



	virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
	{
		FPrimitiveViewRelevance Result;

		Result.bDrawRelevance = IsShown(View);
		Result.bShadowRelevance = IsShadowCast(View);

		bool bUseStaticDrawPath = bPreferStaticDrawPath && AllowStaticDrawPath(View);
		Result.bDynamicRelevance = !bUseStaticDrawPath;
		Result.bStaticRelevance = bUseStaticDrawPath;
#if WITH_EDITOR
		//only check these in the editor
		Result.bEditorVisualizeLevelInstanceRelevance = IsEditingLevelInstanceChild();
		Result.bEditorStaticSelectionRelevance = (IsSelected() || IsHovered());
#endif


		Result.bRenderInMainPass = ShouldRenderInMainPass();
		Result.bUsesLightingChannels = GetLightingChannelMask() != GetDefaultLightingChannelMask();
		Result.bTranslucentSelfShadow = bCastVolumetricTranslucentShadow;
		Result.bRenderCustomDepth = ShouldRenderCustomDepth();
		// Note that this is actually a getter. One may argue that it is poorly named.
		MaterialRelevance.SetPrimitiveViewRelevance(Result);
		Result.bVelocityRelevance = DrawsVelocity() && Result.bOpaque && Result.bRenderInMainPass;

		return Result;
	}

	virtual void UpdatedReferencedMaterials() override
	{
		FBaseDynamicMeshSceneProxy::UpdatedReferencedMaterials();

		// The material relevance may need updating.
		MaterialRelevance = ParentComponent->GetMaterialRelevance(GetScene().GetShaderPlatform());
	}


	virtual void GetLightRelevance(const FLightSceneProxy* LightSceneProxy, bool& bDynamic, bool& bRelevant, bool& bLightMapped, bool& bShadowMapped) const override
	{
		FPrimitiveSceneProxy::GetLightRelevance(LightSceneProxy, bDynamic, bRelevant, bLightMapped, bShadowMapped);
	}

	virtual bool CanBeOccluded() const override
	{
		return !MaterialRelevance.bDisableDepthTest;
	}

	virtual uint32 GetMemoryFootprint(void) const override { return(sizeof(*this) + GetAllocatedSize()); }

	uint32 GetAllocatedSize(void) const { return(FPrimitiveSceneProxy::GetAllocatedSize()); }

	SIZE_T GetTypeHash() const override
	{
		static size_t UniquePointer;
		return reinterpret_cast<size_t>(&UniquePointer);
	}

};







