// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Components/MeshRenderBufferSet.h"


/// ---------------  FMeshRenderBuffferSet ------------------ /// 

void FMeshRenderBufferSet::Upload()
{
	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	if (TriangleCount == 0)
	{
		return;
	}

	InitOrUpdateResource(RHICmdList, &this->PositionVertexBuffer);
	InitOrUpdateResource(RHICmdList, &this->StaticMeshVertexBuffer);
	InitOrUpdateResource(RHICmdList, &this->ColorVertexBuffer);

	FLocalVertexFactory::FDataType Data;
	this->PositionVertexBuffer.BindPositionVertexBuffer(&this->VertexFactory, Data);
	this->StaticMeshVertexBuffer.BindTangentVertexBuffer(&this->VertexFactory, Data);
	this->StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(&this->VertexFactory, Data);
	// currently no lightmaps support
	//this->StaticMeshVertexBuffer.BindLightMapVertexBuffer(&this->VertexFactory, Data, LightMapIndex);
	this->ColorVertexBuffer.BindColorVertexBuffer(&this->VertexFactory, Data);
	this->VertexFactory.SetData(RHICmdList, Data);

	InitOrUpdateResource(RHICmdList, &this->VertexFactory);
	PositionVertexBuffer.InitResource(RHICmdList);
	StaticMeshVertexBuffer.InitResource(RHICmdList);
	ColorVertexBuffer.InitResource(RHICmdList);
	VertexFactory.InitResource(RHICmdList);

	if (IndexBuffer.Indices.Num() > 0)
	{
		IndexBuffer.InitResource(RHICmdList);
	}
	if (bEnableSecondaryIndexBuffer && SecondaryIndexBuffer.Indices.Num() > 0)
	{
		SecondaryIndexBuffer.InitResource(RHICmdList);
	}

	InvalidateRayTracingData();
	ValidateRayTracingData();		// currently we are immediately validating. This may be revisited in future.
}


void FMeshRenderBufferSet::UploadVertexUpdate(bool bPositions, bool bMeshAttribs, bool bColors)
{
	// todo: look at calls to this function, it seems possible that TransferVertexUpdateToGPU
	// could be used instead (which should be somewhat more efficient?). It's not clear if there
	// are any situations where we would change vertex buffer size w/o also updating the index
	// buffers (in which case we are fully rebuilding the buffers...)

	if (TriangleCount == 0)
	{
		return;
	}

	FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

	if (bPositions)
	{
		InitOrUpdateResource(RHICmdList, &this->PositionVertexBuffer);
	}
	if (bMeshAttribs)
	{
		InitOrUpdateResource(RHICmdList, &this->StaticMeshVertexBuffer);
	}
	if (bColors)
	{
		InitOrUpdateResource(RHICmdList, &this->ColorVertexBuffer);
	}

	FLocalVertexFactory::FDataType Data;
	this->PositionVertexBuffer.BindPositionVertexBuffer(&this->VertexFactory, Data);
	this->StaticMeshVertexBuffer.BindTangentVertexBuffer(&this->VertexFactory, Data);
	this->StaticMeshVertexBuffer.BindPackedTexCoordVertexBuffer(&this->VertexFactory, Data);
	this->ColorVertexBuffer.BindColorVertexBuffer(&this->VertexFactory, Data);
	this->VertexFactory.SetData(RHICmdList, Data);

	InitOrUpdateResource(RHICmdList, &this->VertexFactory);

	InvalidateRayTracingData();
	ValidateRayTracingData();		// currently we are immediately validating. This may be revisited in future.
}


void FMeshRenderBufferSet::TransferVertexUpdateToGPU(FRHICommandListBase& RHICmdList, bool bPositions, bool bNormals, bool bTexCoords, bool bColors)
{
	if (TriangleCount == 0)
	{
		return;
	}

	if (bPositions)
	{
		FPositionVertexBuffer& VertexBuffer = this->PositionVertexBuffer;
		void* VertexBufferData = RHICmdList.LockBuffer(VertexBuffer.VertexBufferRHI, 0, VertexBuffer.GetNumVertices() * VertexBuffer.GetStride(), RLM_WriteOnly);
		FMemory::Memcpy(VertexBufferData, VertexBuffer.GetVertexData(), VertexBuffer.GetNumVertices() * VertexBuffer.GetStride());
		RHICmdList.UnlockBuffer(VertexBuffer.VertexBufferRHI);
	}
	if (bNormals)
	{
		FStaticMeshVertexBuffer& VertexBuffer = this->StaticMeshVertexBuffer;
		void* VertexBufferData = RHICmdList.LockBuffer(VertexBuffer.TangentsVertexBuffer.VertexBufferRHI, 0, VertexBuffer.GetTangentSize(), RLM_WriteOnly);
		FMemory::Memcpy(VertexBufferData, VertexBuffer.GetTangentData(), VertexBuffer.GetTangentSize());
		RHICmdList.UnlockBuffer(VertexBuffer.TangentsVertexBuffer.VertexBufferRHI);
	}
	if (bColors)
	{
		FColorVertexBuffer& VertexBuffer = this->ColorVertexBuffer;
		void* VertexBufferData = RHICmdList.LockBuffer(VertexBuffer.VertexBufferRHI, 0, VertexBuffer.GetNumVertices() * VertexBuffer.GetStride(), RLM_WriteOnly);
		FMemory::Memcpy(VertexBufferData, VertexBuffer.GetVertexData(), VertexBuffer.GetNumVertices() * VertexBuffer.GetStride());
		RHICmdList.UnlockBuffer(VertexBuffer.VertexBufferRHI);
	}
	if (bTexCoords)
	{
		FStaticMeshVertexBuffer& VertexBuffer = this->StaticMeshVertexBuffer;
		void* VertexBufferData = RHICmdList.LockBuffer(VertexBuffer.TexCoordVertexBuffer.VertexBufferRHI, 0, VertexBuffer.GetTexCoordSize(), RLM_WriteOnly);
		FMemory::Memcpy(VertexBufferData, VertexBuffer.GetTexCoordData(), VertexBuffer.GetTexCoordSize());
		RHICmdList.UnlockBuffer(VertexBuffer.TexCoordVertexBuffer.VertexBufferRHI);
	}

	InvalidateRayTracingData();
	ValidateRayTracingData();		// currently we are immediately validating. This may be revisited in future.
}



/// ---------------  FMeshRenderBufferSetConversionUtil ------------------ /// 


void FMeshRenderBufferSetConversionUtil::UpdateSecondaryTriangleBuffer(FMeshRenderBufferSet* RenderBuffers,
																       const FDynamicMesh3* Mesh,
																       bool bDuplicate)
{
	if (ensure(bUseSecondaryTriBuffers == true && RenderBuffers->Triangles.IsSet()) == false)
	{
		return;
	}

	const TArray<int32>& TriangleIDs = RenderBuffers->Triangles.GetValue();
	int NumTris = TriangleIDs.Num();
	TArray<uint32>& Indices = RenderBuffers->IndexBuffer.Indices;
	TArray<uint32>& SecondaryIndices = RenderBuffers->SecondaryIndexBuffer.Indices;

	RenderBuffers->SecondaryIndexBuffer.Indices.Reset();
	if (bDuplicate == false)
	{
		RenderBuffers->IndexBuffer.Indices.Reset();
	}
	for (int k = 0; k < NumTris; ++k)
	{
		int TriangleID = TriangleIDs[k];
		bool bInclude = SecondaryTriFilterFunc(Mesh, TriangleID);
		if (bInclude)
		{
			SecondaryIndices.Add(3 * k);
			SecondaryIndices.Add(3 * k + 1);
			SecondaryIndices.Add(3 * k + 2);
		}
		else if (bDuplicate == false)
		{
			Indices.Add(3 * k);
			Indices.Add(3 * k + 1);
			Indices.Add(3 * k + 2);
		}
	}
}

void FMeshRenderBufferSetConversionUtil::RecomputeRenderBufferTriangleIndexSets( FMeshRenderBufferSet* RenderBuffers,
																		         const FDynamicMesh3* Mesh)
{
	if (RenderBuffers->TriangleCount == 0)
	{
		return;
	}
	if (ensure(RenderBuffers->Triangles.IsSet() && RenderBuffers->Triangles->Num() > 0) == false)
	{
		return;
	}

	//bool bDuplicate = false;		// flag for future use, in case we want to draw all triangles in primary and duplicates in secondary...
	RenderBuffers->IndexBuffer.Indices.Reset();
	RenderBuffers->SecondaryIndexBuffer.Indices.Reset();

	TArray<uint32>& Indices = RenderBuffers->IndexBuffer.Indices;
	TArray<uint32>& SecondaryIndices = RenderBuffers->SecondaryIndexBuffer.Indices;
	const TArray<int32>& TriangleIDs = RenderBuffers->Triangles.GetValue();

	int NumTris = TriangleIDs.Num();
	for (int k = 0; k < NumTris; ++k)
	{
		int TriangleID = TriangleIDs[k];
		bool bInclude = SecondaryTriFilterFunc(Mesh, TriangleID);
		if (bInclude)
		{
			SecondaryIndices.Add(3 * k);
			SecondaryIndices.Add(3 * k + 1);
			SecondaryIndices.Add(3 * k + 2);
		}
		else // if (bDuplicate == false)
		{
			Indices.Add(3 * k);
			Indices.Add(3 * k + 1);
			Indices.Add(3 * k + 2);
		}
	}
}


/// ---------------  ToMeshRenderBufferSet ------------------ /// 

void FDynamicMeshComponentToMeshRenderBufferSet::Convert(UDynamicMeshComponent& DynamicMeshCompnent, FMeshRenderBufferSet& MeshRenderBufferSet, bool bUseComponentSettings)
{
	if (bUseComponentSettings)
	{ 
		MeshRenderBufferSetConverter.ColorSpaceTransformMode = DynamicMeshCompnent.GetVertexColorSpaceTransformMode();
		if (DynamicMeshCompnent.GetColorOverrideMode() == EDynamicMeshComponentColorOverrideMode::Constant)
		{
			MeshRenderBufferSetConverter.ConstantVertexColor = DynamicMeshCompnent.GetConstantOverrideColor();
			MeshRenderBufferSetConverter.bIgnoreVertexColors = true;
		}

		MeshRenderBufferSetConverter.bUsePerTriangleNormals = DynamicMeshCompnent.GetFlatShadingEnabled();
	}


	const FDynamicMesh3* Mesh = DynamicMeshCompnent.GetRenderMesh();

	// find suitable overlays
	TArray<const FDynamicMeshUVOverlay*, TInlineAllocator<8>> UVOverlays;
	const FDynamicMeshNormalOverlay* NormalOverlay = nullptr;
	const FDynamicMeshColorOverlay* ColorOverlay = nullptr;
	if (Mesh->HasAttributes())
	{
		const FDynamicMeshAttributeSet* Attributes = Mesh->Attributes();
		NormalOverlay = Attributes->PrimaryNormals();
		UVOverlays.SetNum(Attributes->NumUVLayers());
		for (int32 k = 0; k < UVOverlays.Num(); ++k)
		{
			UVOverlays[k] = Attributes->GetUVLayer(k);
		}
		ColorOverlay = Attributes->PrimaryColors();
	}
	TUniqueFunction<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc = MakeTangentsFunc(DynamicMeshCompnent);

	MeshRenderBufferSetConverter.InitializeBuffersFromOverlays(&MeshRenderBufferSet, Mesh,
		Mesh->TriangleCount(), Mesh->TriangleIndicesItr(),
		UVOverlays, NormalOverlay, ColorOverlay, TangentsFunc);	
}


TUniqueFunction<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> FDynamicMeshComponentToMeshRenderBufferSet::MakeTangentsFunc(UDynamicMeshComponent& DynamicMeshComponent, bool bSkipAutoCompute)
{
	EDynamicMeshComponentTangentsMode TangentsType = DynamicMeshComponent.GetTangentsType();
	if (TangentsType == EDynamicMeshComponentTangentsMode::ExternallyProvided)
	{
		UE::Geometry::FDynamicMesh3* RenderMesh = DynamicMeshComponent.GetRenderMesh();
		// If the RenderMesh has tangents, use them. Otherwise we fall back to the orthogonal basis, below.
		if (RenderMesh && RenderMesh->HasAttributes() && RenderMesh->Attributes()->HasTangentSpace())
		{
			UE::Geometry::FDynamicMeshTangents Tangents(RenderMesh);
			return [Tangents](int VertexID, int TriangleID, int TriVtxIdx, const FVector3f& Normal, FVector3f& TangentX, FVector3f& TangentY) -> void
				{
					Tangents.GetTangentVectors(TriangleID, TriVtxIdx, Normal, TangentX, TangentY);
				};
		}
	}
	else if (TangentsType == EDynamicMeshComponentTangentsMode::AutoCalculated && bSkipAutoCompute == false)
	{
		const UE::Geometry::FMeshTangentsf* Tangents = DynamicMeshComponent.GetAutoCalculatedTangents();
		if (Tangents != nullptr)
		{
			return [Tangents](int VertexID, int TriangleID, int TriVtxIdx, const FVector3f& Normal, FVector3f& TangentX, FVector3f& TangentY) -> void
				{
					Tangents->GetTriangleVertexTangentVectors(TriangleID, TriVtxIdx, TangentX, TangentY);
				};
		}
	}

	// fallback to orthogonal basis
	return [](int VertexID, int TriangleID, int TriVtxIdx, const FVector3f& Normal, FVector3f& TangentX, FVector3f& TangentY) -> void
		{
			UE::Geometry::VectorUtil::MakePerpVectors(Normal, TangentX, TangentY);
		};
}

