// Copyright Epic Games, Inc. All Rights Reserved. 

#include "Components/MeshRenderDecomposition.h"
#include "DynamicMesh/DynamicMesh3.h"
#include "DynamicMesh/DynamicMeshAttributeSet.h"
#include "DynamicMesh/DynamicMeshAABBTree3.h"
#include "Async/ParallelFor.h"
#include "ComponentSourceInterfaces.h"
#include "TargetInterfaces/MaterialProvider.h"

using namespace UE::Geometry;

void FMeshRenderDecomposition::BuildAssociations(const FDynamicMesh3* Mesh)
{
	TriangleToGroupMap.SetNum(Mesh->MaxTriangleID());

	ParallelFor(Groups.Num(), [&](int32 GroupIndex) 
	{
		if (IsGroup(GroupIndex))
		{
			const FGroup& Group = GetGroup(GroupIndex);
			for (int32 tid : Group.Triangles)
			{
				TriangleToGroupMap[tid] = GroupIndex;
			}
		}
	});
}




void FMeshRenderDecomposition::BuildMaterialDecomposition(const FDynamicMesh3* Mesh, const FComponentMaterialSet* MaterialSet, FMeshRenderDecomposition& Decomp)
{
	// always have at least one material
	FComponentMaterialSet LocalMaterials;
	LocalMaterials.Materials.Add(nullptr);
	const FComponentMaterialSet* UseMaterials = (ensure(MaterialSet != nullptr)) ? MaterialSet : &LocalMaterials;

	// may not have MaterialID, in that case all triangles will be material 0
	const FDynamicMeshMaterialAttribute* MaterialID = Mesh->HasAttributes() ? Mesh->Attributes()->GetMaterialID() : nullptr;

	int32 NumMaterials = UseMaterials->Materials.Num();
	Decomp.Initialize(NumMaterials);
	for (int32 k = 0; k < NumMaterials; ++k)
	{
		FMeshRenderDecomposition::FGroup& Group = Decomp.GetGroup(k);
		Group.Material = UseMaterials->Materials[k];
	}

	TArray<int32> InvalidTris;

	for (int32 tid : Mesh->TriangleIndicesItr())
	{
		int32 MatIdx = 0;
		if (MaterialID)
		{
			MaterialID->GetValue(tid, &MatIdx);
		}
		if (MatIdx < NumMaterials)
		{
			FMeshRenderDecomposition::FGroup& Group = Decomp.GetGroup(MatIdx);
			Group.Triangles.Add(tid);
		}
		else
		{
			InvalidTris.Add(tid);		// MaterialID refers to missing material
		}
	}

	// If any triangles were not allocated to a specific material group, put them into a new group.
	// Possibly these will not be rendered correctly by higher-level code, but at least we can
	// guarantee that they are not lost here
	if (InvalidTris.Num() > 0)
	{
		int32 NewGroup = Decomp.AppendGroup();
		FMeshRenderDecomposition::FGroup& Group = Decomp.GetGroup(NewGroup);
		Group.Triangles = MoveTemp(InvalidTris);
		Group.Material = nullptr;
	}
}






static void CollectSubDecomposition(
	const FDynamicMesh3* Mesh, 
	const TArray<int32>& Triangles, 
	UMaterialInterface* Material, 
	FMeshRenderDecomposition& Decomp, 
	int32 MaxChunkSize,
	FCriticalSection& DecompLock)
{
	int32 MaxTrisPerGroup = MaxChunkSize;
	if (Triangles.Num() < MaxTrisPerGroup)
	{
		DecompLock.Lock();
		int32 i = Decomp.AppendGroup();
		FMeshRenderDecomposition::FGroup& Group = Decomp.GetGroup(i);
		DecompLock.Unlock();
		Group.Triangles = Triangles;
		Group.Material = Material;
		return;
	}

	FDynamicMeshAABBTree3 Spatial(Mesh, false);
	Spatial.SetBuildOptions(MaxTrisPerGroup);
	Spatial.Build(Triangles);

	TArray<int32> ActiveSet;
	TArray<int32> SpillSet;

	FDynamicMeshAABBTree3::FTreeTraversal Collector;
	Collector.BeginBoxTrianglesF = [&](int, int)
	{
		ActiveSet.Reset();
		ActiveSet.Reserve(MaxTrisPerGroup);
	};
	Collector.NextTriangleF = [&](int32 TriangleID)
	{
		ActiveSet.Add(TriangleID);
	};
	Collector.EndBoxTrianglesF = [&](int)
	{
		if (ActiveSet.Num() > 0)
		{
			if (ActiveSet.Num() < 100)
			{
				SpillSet.Append(ActiveSet);
			}
			else
			{
				DecompLock.Lock();
				int32 i = Decomp.AppendGroup();
				FMeshRenderDecomposition::FGroup& Group = Decomp.GetGroup(i);
				DecompLock.Unlock();
				Group.Triangles = MoveTemp(ActiveSet);
				Group.Material = Material;
			}
		}
	};
	Spatial.DoTraversal(Collector);

	if (SpillSet.Num() > 0)
	{
		//UE_LOGF(LogTemp, Warning, "SpillSet Size: %d", SpillSet.Num());

		DecompLock.Lock();
		int32 i = Decomp.AppendGroup();
		FMeshRenderDecomposition::FGroup& Group = Decomp.GetGroup(i);
		DecompLock.Unlock();
		Group.Triangles = MoveTemp(SpillSet);
		Group.Material = Material;
	}
}







void FMeshRenderDecomposition::BuildChunkedDecomposition(const FDynamicMesh3* Mesh, const FComponentMaterialSet* MaterialSet, FMeshRenderDecomposition& Decomp, int32 MaxChunkSize)
{
	TUniquePtr<FMeshRenderDecomposition> MaterialDecomp = MakeUnique<FMeshRenderDecomposition>();
	BuildMaterialDecomposition(Mesh, MaterialSet, *MaterialDecomp);
	int32 NumMatGroups = MaterialDecomp->Num();

	FCriticalSection Lock;
	ParallelFor(NumMatGroups, [&](int32 k)
	{
		const FMeshRenderDecomposition::FGroup& Group = MaterialDecomp->GetGroup(k);
		CollectSubDecomposition(Mesh, Group.Triangles, Group.Material, Decomp, MaxChunkSize, Lock);
	});
}