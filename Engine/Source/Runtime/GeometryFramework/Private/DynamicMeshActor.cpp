// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicMeshActor.h"
#include "Engine/CollisionProfile.h"
#include "MaterialDomain.h"
#include "Materials/Material.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicMeshActor)


#define LOCTEXT_NAMESPACE "ADynamicMeshActor"

ADynamicMeshActor::ADynamicMeshActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DynamicMeshComponent = CreateDefaultSubobject<UDynamicMeshComponent>(TEXT("DynamicMeshComponent"));
	DynamicMeshComponent->SetMobility(EComponentMobility::Movable);
	DynamicMeshComponent->SetGenerateOverlapEvents(false);
	DynamicMeshComponent->SetCollisionProfileName(UCollisionProfile::BlockAll_ProfileName);

	DynamicMeshComponent->CollisionType = ECollisionTraceFlag::CTF_UseDefault;

	DynamicMeshComponent->SetMaterial(0, UMaterial::GetDefaultMaterial(MD_Surface));		// is this necessary?

	SetRootComponent(DynamicMeshComponent);
}


UDynamicMeshPool* ADynamicMeshActor::GetComputeMeshPool()
{
	if (DynamicMeshPool == nullptr && bEnableComputeMeshPool)
	{
		DynamicMeshPool = NewObject<UDynamicMeshPool>();
	}
	return DynamicMeshPool;
}


UDynamicMesh* ADynamicMeshActor::AllocateComputeMesh()
{
	if (bEnableComputeMeshPool)
	{
		UDynamicMeshPool* UsePool = GetComputeMeshPool();
		if (UsePool)
		{
			return UsePool->RequestMesh();
		}
	}

	// if we could not return a pool mesh, allocate a new mesh that isn't owned by the pool
	return NewObject<UDynamicMesh>(this);
}


bool ADynamicMeshActor::ReleaseComputeMesh(UDynamicMesh* Mesh)
{
	if (bEnableComputeMeshPool && Mesh)
	{
		UDynamicMeshPool* UsePool = GetComputeMeshPool();
		if (UsePool != nullptr)
		{
			UsePool->ReturnMesh(Mesh);
			return true;
		}
	}
	return false;
}


void ADynamicMeshActor::ReleaseAllComputeMeshes()
{
	UDynamicMeshPool* UsePool = GetComputeMeshPool();
	if (UsePool)
	{
		UsePool->ReturnAllMeshes();
	}
}

void ADynamicMeshActor::FreeAllComputeMeshes()
{
	UDynamicMeshPool* UsePool = GetComputeMeshPool();
	if (UsePool)
	{
		UsePool->FreeAllMeshes();
	}
}




#undef LOCTEXT_NAMESPACE
