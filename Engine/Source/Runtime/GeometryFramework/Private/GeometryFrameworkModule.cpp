// Copyright Epic Games, Inc. All Rights Reserved.

#include "GeometryFrameworkModule.h"
#include "Components/BaseDynamicMeshComponent.h"
#include "Misc/CoreDelegates.h"

#if WITH_EDITOR

#include "DynamicMeshActor.h"
#include "LevelEditor.h"
#include "Filters/CustomClassFilterData.h"

#endif

#define LOCTEXT_NAMESPACE "FGeometryFrameworkModule"

void FGeometryFrameworkModule::StartupModule()
{
	FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FGeometryFrameworkModule::OnPostEngineInit);

#if WITH_EDITOR
	if (!IsRunningGame() && !IsRunningDedicatedServer())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::LoadModuleChecked<FLevelEditorModule>("LevelEditor");

		// Register the Level Editor Outliner filter for ADynamicMeshActor
		if(TSharedPtr<FFilterCategory> GeometryFilterCategory = LevelEditorModule.GetOutlinerFilterCategory(FLevelEditorOutlinerBuiltInCategories::Geometry()))
		{
			TSharedRef<FCustomClassFilterData> DynamicMeshActorClassData = MakeShared<FCustomClassFilterData>(ADynamicMeshActor::StaticClass(), GeometryFilterCategory, FLinearColor::White);
			LevelEditorModule.AddCustomClassFilterToOutliner(DynamicMeshActorClassData);
		}
	}
#endif
}

void FGeometryFrameworkModule::OnPostEngineInit()
{
	// UBaseDynamicMeshComponent provides some global materials to all instances, rather than
	// directly accessing (eg) GEngine pointers. Initialize those here. 
	UBaseDynamicMeshComponent::InitializeDefaultMaterials();
}

void FGeometryFrameworkModule::ShutdownModule()
{
	FCoreDelegates::GetOnPostEngineInit().RemoveAll(this);
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FGeometryFrameworkModule, GeometryFramework)