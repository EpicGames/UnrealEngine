// Copyright Epic Games, Inc. All Rights Reserved.

#include "UDynamicMesh.h"

#include "Changes/MeshVertexChange.h"
#include "Changes/MeshChange.h"
#include "Changes/MeshReplacementChange.h"
#include "Misc/Base64.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/MemoryReader.h"
#include "UObject/UE5MainStreamObjectVersion.h"
#include "HAL/IConsoleManager.h"
#include "Engine/Engine.h"
#include "Generators/MinimalBoxMeshGenerator.h"
#include "Misc/TransactionCommon.h"


#if WITH_EDITOR
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Notifications/NotificationManager.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(UDynamicMesh)


#define LOCTEXT_NAMESPACE "UDynamicMesh"

using namespace UE::Geometry;


// these cvars are used to support T3D encoding of the internal FDynamicMesh3, see ::ExportCustomProperties() and ::ImportCustomProperties()
static TAutoConsoleVariable<int32> CVarDynamicMeshTextBasedDupeTriangleCountThreshold(
	TEXT("geometry.DynamicMesh.TextBasedDupeTriThreshold"),
	200000,
	TEXT("Triangle count threshold for text-based UDynamicMesh duplication using Base64. Large values are quite slow."));
static TAutoConsoleVariable<int32> CVarDynamicMeshDupeHelperTimeout(
	TEXT("geometry.DynamicMesh.DupeStashTimeout"),
	5*60,
	TEXT("Timeout in seconds for references held by internal UDynamicMesh duplication helper system. See FDynamicMeshCopyHelper."));



namespace UE::Private::UDynamicMeshLocal
{
	static void DisplayCriticalWarningMessage(const FText& InMessage, float ExpireDuration = 5.0f)
	{
#if WITH_EDITOR
		FNotificationInfo Info(InMessage);
		Info.ExpireDuration = ExpireDuration;
		FSlateNotificationManager::Get().AddNotification(Info);
#endif

		UE_LOGF(LogGeometry, Warning, "%ls", *InMessage.ToString());
	}
}


UDynamicMesh::UDynamicMesh(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	InitializeNewMesh();
}


void UDynamicMesh::InitializeNewMesh()
{
	Mesh = MakeUnique<FDynamicMesh3>(EMeshComponents::FaceGroups);
	InitializeMesh();
}

UDynamicMesh* UDynamicMesh::Reset()
{
	FDynamicMeshChangeInfo ChangeInfo;
	ChangeInfo.Type = EDynamicMeshChangeType::GeneralEdit;
	EditMeshInternal([this](FDynamicMesh3& EditMesh)
	{
		check(&EditMesh == Mesh.Get());		// assuming that EditMesh is internal mesh here...
		InitializeMesh();
	}, ChangeInfo);
	return this;
}

UDynamicMesh* UDynamicMesh::ResetToCube()
{
	FDynamicMeshChangeInfo ChangeInfo;
	ChangeInfo.Type = EDynamicMeshChangeType::GeneralEdit;
	EditMeshInternal([this](FDynamicMesh3& EditMesh)
	{
		EditMesh.Clear();
		FMinimalBoxMeshGenerator BoxGen;
		BoxGen.Box = UE::Geometry::FOrientedBox3d(FVector3d::Zero(), 50.0 * FVector3d::One());
		EditMesh = FDynamicMesh3(&BoxGen.Generate());
		EditMesh.EnableTriangleGroups();
		EditMesh.Attributes()->EnableMaterialID();
	}, ChangeInfo);
	return this;
}


void UDynamicMesh::InitializeMesh()
{
	Mesh->Clear();
	Mesh->EnableTriangleGroups();
	Mesh->EnableAttributes();
	Mesh->Attributes()->EnableMaterialID();

	if (MeshGenerator != nullptr && bEnableMeshGenerator)
	{
		MeshGenerator->Generate(*Mesh);
	}
}


bool UDynamicMesh::IsEmpty() const
{
	return Mesh->TriangleCount() == 0;
}

int32 UDynamicMesh::GetTriangleCount() const
{
	return Mesh->TriangleCount();
}


void UDynamicMesh::SetMesh(const UE::Geometry::FDynamicMesh3& MoveMesh)
{
	FDynamicMeshChangeInfo ChangeInfo;
	ChangeInfo.Type = EDynamicMeshChangeType::GeneralEdit;
	EditMeshInternal([&](FDynamicMesh3& EditMesh) {
		EditMesh = MoveMesh;
	}, ChangeInfo);
}

void UDynamicMesh::SetMesh(UE::Geometry::FDynamicMesh3&& MoveMesh)
{
	FDynamicMeshChangeInfo ChangeInfo;
	ChangeInfo.Type = EDynamicMeshChangeType::GeneralEdit;
	EditMeshInternal([&](FDynamicMesh3& EditMesh) {
		EditMesh = MoveTemp(MoveMesh);
	}, ChangeInfo);
}

void UDynamicMesh::ProcessMesh(TFunctionRef<void(const UE::Geometry::FDynamicMesh3&)> ProcessFunc) const
{
	ProcessFunc(GetMeshRef());
}

void UDynamicMesh::EditMesh(TFunctionRef<void(FDynamicMesh3&)> EditFunc,
							EDynamicMeshChangeType ChangeType,
							EDynamicMeshAttributeChangeFlags ChangeFlags,
							bool bDeferChangeEvents)
{
	FDynamicMeshChangeInfo ChangeInfo;
	ChangeInfo.Type = ChangeType;
	ChangeInfo.Flags = ChangeFlags;
	EditMeshInternal(EditFunc, ChangeInfo, bDeferChangeEvents);
}

void UDynamicMesh::EditMeshInternal(TFunctionRef<void(FDynamicMesh3&)> EditFunc, const FDynamicMeshChangeInfo& ChangeInfo, bool bDeferChangeEvents)
{
	if (!bDeferChangeEvents)
	{
		PreMeshChangedEvent.Broadcast(this, ChangeInfo);
	}
	EditFunc(GetMeshRef());

	// Enforce our mesh attribute invariants. This should probably be optional to support compute-only UDynamicMeshes....
	if (Mesh->HasTriangleGroups() == false)
	{
		Mesh->EnableTriangleGroups();
	}
	if (Mesh->HasAttributes() == false)
	{
		Mesh->EnableAttributes();
	}
	if (Mesh->Attributes()->HasMaterialID() == false)
	{
		Mesh->Attributes()->EnableMaterialID();
	}

	if (!bDeferChangeEvents)
	{
		MeshChangedEvent.Broadcast(this, ChangeInfo);
		MeshModifiedBPEvent.Broadcast(this);
	}
}


TUniquePtr<FDynamicMesh3> UDynamicMesh::ExtractMesh()
{
	TUniquePtr<FDynamicMesh3> ReturnMesh = MoveTemp(Mesh);
	InitializeNewMesh();
	return ReturnMesh;
}



void UDynamicMesh::SetMeshGenerator(TObjectPtr<UDynamicMeshGenerator> NewGenerator)
{
	MeshGenerator = NewGenerator;
}

void UDynamicMesh::ClearMeshGenerator()
{
	MeshGenerator = nullptr;
}

void UDynamicMesh::Regenerate()
{
	if (MeshGenerator != nullptr && bEnableMeshGenerator)
	{
		Reset();
	}
}


void UDynamicMesh::PostRealtimeUpdate() 
{ 
	MeshRealtimeUpdateEvent.Broadcast(this); 
}


void UDynamicMesh::ApplyChange(const FMeshVertexChange* Change, bool bRevert)
{
	FDynamicMeshChangeInfo ChangeInfo;
	ChangeInfo.Type = EDynamicMeshChangeType::MeshVertexChange;
	ChangeInfo.VertexChange = Change;
	ChangeInfo.bIsRevertChange = bRevert;

	EditMeshInternal([&](FDynamicMesh3& EditMesh)
	{
		bool bHavePositions = Change->bHaveVertexPositions;
		bool bHaveColors = Change->bHaveVertexColors && EditMesh.HasVertexColors();

		int32 NV = Change->Vertices.Num();
		const TArray<FVector3d>& Positions = (bRevert) ? Change->OldPositions : Change->NewPositions;
		const TArray<FVector3f>& Colors = (bRevert) ? Change->OldColors : Change->NewColors;
		for (int32 k = 0; k < NV; ++k)
		{
			int32 vid = Change->Vertices[k];
			if (EditMesh.IsVertex(vid))
			{
				if (bHavePositions)
				{
					EditMesh.SetVertex(vid, Positions[k]);
				}
				if (bHaveColors)
				{
					EditMesh.SetVertexColor(vid, Colors[k]);
				}
			}
		}

		if (Change->bHaveOverlayNormals && EditMesh.HasAttributes() && EditMesh.Attributes()->PrimaryNormals())
		{
			FDynamicMeshNormalOverlay* Overlay = EditMesh.Attributes()->PrimaryNormals();
			int32 NumNormals = Change->Normals.Num();
			const TArray<FVector3f>& UseNormals = (bRevert) ? Change->OldNormals : Change->NewNormals;
			for (int32 k = 0; k < NumNormals; ++k)
			{
				int32 elemid = Change->Normals[k];
				if (Overlay->IsElement(elemid))
				{
					Overlay->SetElement(elemid, UseNormals[k]);
				}
			}
		}

		if (Change->bHaveOverlayUVs && EditMesh.HasAttributes() && EditMesh.Attributes()->PrimaryUV())
		{
			FDynamicMeshUVOverlay* Overlay = EditMesh.Attributes()->PrimaryUV();
			int32 NumUVs = Change->UVs.Num();
			const TArray<FVector2f>& UseUVs = (bRevert) ? Change->OldUVs : Change->NewUVs;
			for (int32 k = 0; k < NumUVs; ++k)
			{
				int32 elemid = Change->UVs[k];
				if (Overlay->IsElement(elemid))
				{
					Overlay->SetElement(elemid, UseUVs[k]);
				}
			}
		}
	}, ChangeInfo);
}

void UDynamicMesh::ApplyChange(const FMeshChange* Change, bool bRevert)
{
	FDynamicMeshChangeInfo ChangeInfo;
	ChangeInfo.Type = EDynamicMeshChangeType::MeshChange;
	ChangeInfo.MeshChange = Change;
	ChangeInfo.bIsRevertChange = bRevert;

	EditMeshInternal([&](FDynamicMesh3& EditMesh)
	{
		Change->ApplyChangeToMesh(&EditMesh, bRevert);
	}, ChangeInfo);
}

void UDynamicMesh::ApplyChange(const FMeshReplacementChange* Change, bool bRevert)
{
	FDynamicMeshChangeInfo ChangeInfo;
	ChangeInfo.Type = EDynamicMeshChangeType::MeshReplacementChange;
	ChangeInfo.ReplaceChange = Change;
	ChangeInfo.bIsRevertChange = bRevert;

	EditMeshInternal([&](FDynamicMesh3& EditMesh)
	{
		EditMesh.Copy(*Change->GetMesh(bRevert));
	}, ChangeInfo);
}


void UDynamicMesh::Serialize(FArchive& Ar)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UDynamicMesh::Serialize);

	Super::Serialize(Ar);

	// Do not serialize mesh for transactions/diff objects. Mesh data doesn't need to end up in the DiffObject sync so skipping is fine. 
	if (Ar.IsTransacting() || UE::Transaction::DiffUtil::IsGeneratingDiffableObject(Ar))
	{
		return;
	}

	Ar.UsingCustomVersion(FUE5MainStreamObjectVersion::GUID);

	if (Ar.IsLoading())
	{
		InitializeNewMesh();
	}
	Ar << *Mesh;
}






/**
 * This is an internal singleton used to support UDynamicMesh::ExportCustomProperties() and UDynamicMesh::ImportCustomProperties() below.
 * 
 */
namespace UE 
{
namespace Local
{

class FDynamicMeshCopyHelper
{
protected:
	struct FStashedMesh
	{
		TSoftObjectPtr<UDynamicMesh> SourceMesh;
		FDateTime Timestamp;
	};

	static TMap<int32, FStashedMesh> StashedMeshes;
	static FRandomStream KeyGenerator;
public:

	static void Initialize()
	{
		static bool bInitialized = false;
		if (!bInitialized)
		{
			KeyGenerator = FRandomStream( (int32)FDateTime::UtcNow().ToUnixTimestamp() );
			bInitialized = true;
		}
	}

	static void DiscardExpiredMeshes()
	{
		const int32 ExpiryTimeoutInSeconds = CVarDynamicMeshDupeHelperTimeout.GetValueOnGameThread();

		TArray<int32> ToRemove;
		for (TPair<int32, FStashedMesh>& Pair : StashedMeshes)
		{
			if ((FDateTime::Now() - Pair.Value.Timestamp).GetTotalSeconds() > (double)ExpiryTimeoutInSeconds)
			{
				ToRemove.Add(Pair.Key);
			}
		}
		for (int32 id : ToRemove)
		{
			StashedMeshes.Remove(id);
		}
	}

	static int32 StashMeshReference(UDynamicMesh* SourceMesh)
	{
		Initialize();
		int32 MeshKey = FMath::Abs((int32)KeyGenerator.GetUnsignedInt());

		FStashedMesh NewMesh;
		NewMesh.Timestamp = FDateTime::Now();
		NewMesh.SourceMesh = SourceMesh;
		StashedMeshes.Add(MeshKey, MoveTemp(NewMesh));

		DiscardExpiredMeshes();
		return MeshKey;
	}

	static TUniquePtr<FDynamicMesh3> ExtractStashedMesh(int32 MeshKey)
	{
		TUniquePtr<FDynamicMesh3> ReturnMesh;
		FStashedMesh* Found = StashedMeshes.Find(MeshKey);
		if (Found != nullptr && Found->SourceMesh.IsValid())
		{
			ReturnMesh = MakeUnique<FDynamicMesh3>();
			UDynamicMesh* SourceMesh = Found->SourceMesh.Get();
			if (SourceMesh)
			{
				// should we Cast<UDynamicMesh> here to make sure this is still the same UClass? (due to recyling)
				SourceMesh->ProcessMesh([&](const FDynamicMesh3& EditMesh)
				{
					*ReturnMesh = EditMesh;
				});
			}
		}

		DiscardExpiredMeshes();
		return ReturnMesh;
	}

};
TMap<int32, FDynamicMeshCopyHelper::FStashedMesh> FDynamicMeshCopyHelper::StashedMeshes;
FRandomStream FDynamicMeshCopyHelper::KeyGenerator;

} } // end namespace UE::Local


void UDynamicMesh::ExportCustomProperties(FOutputDevice& Out, uint32 Indent)
{
	// ignore empty meshes
	if (Mesh->TriangleCount() == 0 && Mesh->VertexCount() == 0)
	{
		return;
	}

	// In the Editor, Copy/Paste of Actors and Components is not based on serialization,
	// but rather on T3D structured text records. This is what allows things to be copied
	// between Editor sessions, for example. Duplicate functionality in the Editor is implemented
	// as Copy and Paste, so for Duplicate to work, any non-UProperty data must be serialized via text.
	//
	// This poses a problem for large data, such as large meshes. Serializing a mesh with millions of
	// triangles as structured text is very expensive. An alternative is to serialize it to binary and 
	// then use Base64 encoding. This is implemented below, however even that is quite slow. 
	// 
	// So, we employ a second strategy (read: hack), of just passing the source UObject pointer 
	// via text. Instead of directly sending the pointer, we store it in FDynamicMeshCopyHelper, and pass
	// an integer key instead. This allows for somewhat better handling, eg the copy helper could for example
	// keep the UDynamicMeshes alive (so far this has not been necessary) and the keys are randomized so
	// even in the unlikely even that two Editor instances end up with the same keys, it would just
	// result in an correct pasted object, rather than accessing garbage pointers.
	// 
	// FDynamicMeshCopyHelper does attempt to discard "old" references, the CVar geometry.DynamicMesh.DupeStashTimeout 
	// controls the definition of old (currenly 5 minutes). One effect this can have is if one does a
	// copy and then a paste after the timeout, the mesh will not be found. This could also be problematic
	// for large full-scene copies of hundreds of objects that takes longer than the timeout (resolvable via the CVar)
	// 
	// Obviously this does not work between Editor sessions. So we also optionally do a Base64 binary encoding if
	// the mesh triangle count is below a CVar threshold (geometry.DynamicMesh.TextBasedDupeTriThreshold) 
	// defaulting to 1000. Larger meshes can be supported by increasing the CVar value if need be.
	// 
	// If the mesh is not found in the FDynamicMeshCopyHelper, and is too large to text-copy, then instead
	// of leaving an empty mesh, we emit a cube, as empty meshes can be problematic. A warning is also
	// printed to the Output Log, pointing the user to the CVars.
	// 
	// Possible todos: UObject Reuse/Recycling
	//

	Super::ExportCustomProperties(Out, Indent);
	Out.Logf(TEXT("%sCustomProperties "), FCString::Spc(Indent));
	Out.Logf(TEXT("MeshData "));

	// stash copy to circumvent expensive T3D generation/parsing
	int32 StashedKey = UE::Local::FDynamicMeshCopyHelper::StashMeshReference(this);
	Out.Logf(TEXT("MESHKEY=%d "), StashedKey);

	if (Mesh->TriangleCount() < CVarDynamicMeshTextBasedDupeTriangleCountThreshold.GetValueOnGameThread())
	{
		// serialize our mesh
		TArray<uint8> MeshWriteBuffer;
		FMemoryWriter MemWriter(MeshWriteBuffer);
		Mesh->Serialize(MemWriter);

		FString Base64String = FBase64::Encode(MeshWriteBuffer);
		//TArray<uint8> MeshReadBuffer;
		//ensure(FBase64::Decode(Base64String, MeshReadBuffer));		// test decode

		// Base64 encoding uses the '/' character, but T3D interprets '//' as some kind of
		// terminator (?). If it occurs then the string passed to ImportCustomProperties() will
		// come back as full of nullptrs. So we will swap in '-' here, and swap back to '/' in ImportCustomProperties()
		for (int32 k = 0; k < Base64String.Len(); ++k)
		{
			if (Base64String[k] == '/')
			{
				Base64String[k] = '-';
			}
		}

		Out.Logf(TEXT("MESHDATALEN=%d MESHDATA=%s"), Base64String.Len(), *Base64String);
	}

	Out.Logf(TEXT("\r\n"));
}


void UDynamicMesh::ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn)
{
	Super::ImportCustomProperties(SourceText, Warn);

	if (FParse::Command(&SourceText, TEXT("MeshData")))	
	{
		static const TCHAR MeshKeyToken[] = TEXT("MESHKEY=");
		const TCHAR* FoundMeshKeyStart = FCString::Strifind(SourceText, MeshKeyToken);
		if (FoundMeshKeyStart)
		{
			SourceText = FoundMeshKeyStart + FCString::Strlen(MeshKeyToken);
			int32 MeshKey = FCString::Atoi(SourceText);
			TUniquePtr<FDynamicMesh3> FoundMesh = UE::Local::FDynamicMeshCopyHelper::ExtractStashedMesh(MeshKey);
			if (FoundMesh.IsValid() && FoundMesh->TriangleCount() > 0)
			{
				InitializeNewMesh();
				SetMesh(MoveTemp(*FoundMesh));
				return;
			}
		}

		static const TCHAR MeshDataLenToken[] = TEXT("MESHDATALEN=");
		const TCHAR* FoundMeshDataLenStart = FCString::Strifind(SourceText, MeshDataLenToken);
		if (FoundMeshDataLenStart)
		{
			SourceText = FoundMeshDataLenStart + FCString::Strlen(MeshDataLenToken);
			int32 MeshDataLen = FCString::Atoi(SourceText);

			static const TCHAR MeshDataToken[] = TEXT("MESHDATA=");
			const TCHAR* FoundMeshDataStart = FCString::Strifind(SourceText, MeshDataToken);
			if (FoundMeshDataStart)
			{
				SourceText = FoundMeshDataStart + FCString::Strlen(MeshDataToken);
				FString MeshData = FString::ConstructFromPtrSize(SourceText, MeshDataLen);

				// fix-up the hack applied to the Base64-encoded string in ExportCustomProperties()
				for (int32 k = 0; k < MeshData.Len(); ++k)
				{
					if (MeshData[k] == '-')
					{
						MeshData[k] = '/';
					}
				}

				TArray<uint8> MeshReadBuffer;
				bool bDecoded = FBase64::Decode(MeshData, MeshReadBuffer);
				if (bDecoded)
				{
					FMemoryReader MemReader(MeshReadBuffer);
					FDynamicMesh3 NewMesh;
					NewMesh.Serialize(MemReader);

					InitializeNewMesh();
					SetMesh(MoveTemp(NewMesh));
					return;
				}
			}
		}

		// if we got here we failed. Rather than produce an empty mesh, we generate a small cube
		UE::Private::UDynamicMeshLocal::DisplayCriticalWarningMessage(LOCTEXT("DynamicMeshPasteFailed", "Dynamic Mesh paste failed! See log for details."));
		UE_LOGF(LogGeometry, Warning, "UDynamicMesh text-based property serialization incomplete, generating box as placeholder. Try increasing geometry.DynamicMesh.TextBasedDupeTriThreshold, or geometry.DynamicMesh.DupeStashTimeout.")
		FMinimalBoxMeshGenerator BoxGen;
		BoxGen.Box = UE::Geometry::FOrientedBox3d(FVector3d::Zero(), 50.0 * FVector3d::One());
		FDynamicMesh3 GenMesh(&BoxGen.Generate());
		GenMesh.Attributes()->EnableMaterialID();
		SetMesh(MoveTemp(GenMesh));
	}
}





//
// Pool support for blueprints
//

static TAutoConsoleVariable<int32> CVarDynamicMeshPoolMaxPoolSizeThreshold(
	TEXT("geometry.DynamicMesh.MaxPoolSize"),
	1000,
	TEXT("Maximum number of meshes a UDynamicMeshPool will allow to be in the pool before running garbage collection"));



UDynamicMesh* UDynamicMeshPool::RequestMesh()
{
	FScopeLock PoolScopeLock(&PoolAccessCs);
	if (CachedMeshes.Num() > 0)
	{
		return CachedMeshes.Pop(EAllowShrinking::No);
	}
	UDynamicMesh* NewMesh = NewObject<UDynamicMesh>();

	// If we have allocated more meshes than our safety threshold, drop our holds on the existing meshes.
	// This will allow them to be garbage-collected (eventually)
	if (!ensure(AllCreatedMeshes.Num() < CVarDynamicMeshPoolMaxPoolSizeThreshold.GetValueOnAnyThread()))
	{
		UE_LOGF(LogGeometry, Warning, "UDynamicMeshPool Threshold of %d Allocated Meshes exceeded! Releasing references to all current meshes and forcing a garbage collection.", CVarDynamicMeshPoolMaxPoolSizeThreshold.GetValueOnAnyThread());
		AllCreatedMeshes.Reset();
		// We generally expect the mesh pool to be used from game-thread blueprint code, but in case it is not, take extra care that ForceGarbageCollection is only called from the game thread
		if (IsInGameThread())
		{
			GEngine->ForceGarbageCollection(true);
		}
		else
		{
			AsyncTask(ENamedThreads::GameThread, []()
			{
				GEngine->ForceGarbageCollection(true);
			});
		}
	}

	AllCreatedMeshes.Add(NewMesh);
	return NewMesh;
}



void UDynamicMeshPool::ReturnMesh(UDynamicMesh* Mesh)
{
	FScopeLock PoolScopeLock(&PoolAccessCs);
	if ( ensure(Mesh) && ensure(AllCreatedMeshes.Contains(Mesh)) )
	{
		Mesh->Reset();
		if (ensure(CachedMeshes.Contains(Mesh) == false))
		{
			CachedMeshes.Add(Mesh);
		}
	}
}


void UDynamicMeshPool::ReturnAllMeshes()
{
	FScopeLock PoolScopeLock(&PoolAccessCs);
	CachedMeshes = AllCreatedMeshes;
	for (UDynamicMesh* Mesh : CachedMeshes)
	{
		if (Mesh)
		{
			Mesh->Reset();
		}
	}

	// TODO: this may be vestigial code, unclear how it could be hit
	int32 Removed = CachedMeshes.RemoveAll([](UDynamicMesh* Mesh) { return Mesh == nullptr; });
	ensure(Removed == 0);
}

void UDynamicMeshPool::FreeAllMeshes()
{
	FScopeLock PoolScopeLock(&PoolAccessCs);
	CachedMeshes.Reset();
	AllCreatedMeshes.Reset();
}

#undef LOCTEXT_NAMESPACE