// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Changes/MeshRegionChange.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMeshChangeTracker.h"
#include "InteractiveToolChange.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "VectorTypes.h"

#include "MeshChange.generated.h"

class UObject;
namespace UE { namespace Geometry { class FDynamicMesh3; } }

//class FDynamicMeshChange;		// need to refactor this out of DynamicMeshChangeTracker




/**
 * FMeshChange represents an undoable change to a FDynamicMesh3.
 * Currently only valid to call Apply/Revert when the Object is a one of several components backed by FDynamicMesh: UDynamicMeshComponent, UOctreeDynamicMeshComponent, UPreviewMesh
 */
class FMeshChange : public FMeshRegionChangeBase
{
public:
	GEOMETRYFRAMEWORK_API FMeshChange();
	GEOMETRYFRAMEWORK_API FMeshChange(TUniquePtr<UE::Geometry::FDynamicMeshChange> DynamicMeshChangeIn);
	GEOMETRYFRAMEWORK_API ~FMeshChange();

	TUniquePtr<UE::Geometry::FDynamicMeshChange> DynamicMeshChange;

	/** This function is called on Apply and Revert (last argument is true on Apply)*/
	TFunction<void(FMeshChange*, UObject*, bool)> OnChangeAppliedFunc;

	/** Makes the change to the object */
	GEOMETRYFRAMEWORK_API virtual void Apply(UObject* Object) override;

	/** Reverts change to the object */
	GEOMETRYFRAMEWORK_API virtual void Revert(UObject* Object) override;

	/** FRegionMeshChangeBase API -- builds an array of all vertices changed and passes to the ProcessFn lambda */
	GEOMETRYFRAMEWORK_API virtual void ProcessChangeVertices(const UE::Geometry::FDynamicMesh3* ChangedMesh, TFunctionRef<void(TConstArrayView<int32>)> ProcessFn, bool bRevert) const override;

	/** Describes this change (for debugging) */
	GEOMETRYFRAMEWORK_API virtual FString ToString() const override;


	/** 
	 * Apply embedded DynamicMeshChange to given Mesh. This function is for 
	 * change-targets to call, when passed a FMeshChange to apply to a Mesh they own. 
	 * This allows FMeshChange subclasses to customize the change behavior if necessary.
	 * The default behavior just forwards the call to DynamicMeshChange->Apply(Mesh, bRevert).
	 */
	GEOMETRYFRAMEWORK_API virtual void ApplyChangeToMesh(UE::Geometry::FDynamicMesh3* Mesh, bool bRevert) const;
};





UINTERFACE(MinimalAPI)
class UMeshCommandChangeTarget : public UInterface
{
	GENERATED_BODY()
};
/**
 * IMeshCommandChangeTarget is an interface which is used to apply a mesh change
 */
class IMeshCommandChangeTarget
{
	GENERATED_BODY()
public:
	virtual void ApplyChange(const FMeshChange* Change, bool bRevert) = 0;
};


