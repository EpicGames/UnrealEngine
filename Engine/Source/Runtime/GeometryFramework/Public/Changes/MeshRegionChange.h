// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InteractiveToolChange.h"

namespace UE::Geometry { class FDynamicMesh3; }

// Base class for mesh changes that apply to a selection of the mesh
class FMeshRegionChangeBase : public FToolCommandChange
{
public:
	// @param ChangedMesh The mesh after the change has been applied (or undone, if bRevert is true)
	// @param ProcessFn The callback that will be called with the vertices affected by this mesh change
	// @param bRevert Whether the changes is an undo
	virtual void ProcessChangeVertices(const UE::Geometry::FDynamicMesh3* ChangedMesh, TFunctionRef<void(TConstArrayView<int32>)> ProcessFn, bool bRevert) const = 0;
};
