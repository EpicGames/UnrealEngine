// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Changes/MeshRegionChange.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "GeometryBase.h"
#include "InteractiveToolChange.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "UObject/Interface.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MeshReplacementChange.generated.h"

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);
class UObject;


/**
 * FMeshReplacementChange represents an undoable *complete* change to a FDynamicMesh3.
 * Currently only valid to call Apply/Revert when the Object is a UDynamicMeshComponent
 */
class FMeshReplacementChange : public FMeshRegionChangeBase
{
	TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> Before, After;

public:
	GEOMETRYFRAMEWORK_API FMeshReplacementChange();
	GEOMETRYFRAMEWORK_API FMeshReplacementChange(TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> Before, TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe> After);

	const TSharedPtr<const FDynamicMesh3, ESPMode::ThreadSafe>& GetMesh(bool bRevert) const
	{
		return bRevert ? Before : After;
	}

	/** This function is called on Apply and Revert (last argument is true on Apply)*/
	TFunction<void(FMeshReplacementChange*, UObject*, bool)> OnChangeAppliedFunc;

	/** Makes the change to the object */
	GEOMETRYFRAMEWORK_API virtual void Apply(UObject* Object) override;

	/** Reverts change to the object */
	GEOMETRYFRAMEWORK_API virtual void Revert(UObject* Object) override;

	/** FRegionMeshChangeBase API -- passes all mesh vertex IDs as an array to the ProcessFn lambda */
	GEOMETRYFRAMEWORK_API virtual void ProcessChangeVertices(const UE::Geometry::FDynamicMesh3* ChangedMesh, TFunctionRef<void(TConstArrayView<int32>)> ProcessFn, bool bRevert) const override;

	/** Describes this change (for debugging) */
	GEOMETRYFRAMEWORK_API virtual FString ToString() const override;
};





UINTERFACE(MinimalAPI)
class UMeshReplacementCommandChangeTarget : public UInterface
{
	GENERATED_BODY()
};
/**
 * IMeshReplacementCommandChangeTarget is an interface which is used to apply a mesh replacement change
 */
class IMeshReplacementCommandChangeTarget
{
	GENERATED_BODY()
public:
	virtual void ApplyChange(const FMeshReplacementChange* Change, bool bRevert) = 0;
};


