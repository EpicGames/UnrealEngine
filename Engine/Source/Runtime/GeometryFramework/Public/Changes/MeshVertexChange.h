// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Changes/MeshRegionChange.h"
#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "InteractiveToolChange.h"
#include "VectorTypes.h"
#include "MeshVertexChange.generated.h"

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);

/**
 * FMeshVertexChange represents an reversible change to a set of vertex positions, normals, colors and UVs
 * Currently only a UDynamicMeshComponent target is supported.
 * 
 * @todo support optionally storing old/new normals and tangents
 * @todo support applying to a StaticMeshComponent/MeshDescription ?
 */
class FMeshVertexChange : public FMeshRegionChangeBase
{
public:
	bool bHaveVertexPositions = true;
	bool bHaveVertexColors = false;
	TArray<int32> Vertices;
	TArray<FVector3d> OldPositions;
	TArray<FVector3d> NewPositions;
	TArray<FVector3f> OldColors;
	TArray<FVector3f> NewColors;

	bool bHaveOverlayNormals = false;
	TArray<int32> Normals;
	TArray<FVector3f> OldNormals;
	TArray<FVector3f> NewNormals;

	bool bHaveOverlayUVs = false;
	TArray<int32> UVs;
	TArray<FVector2f> OldUVs;
	TArray<FVector2f> NewUVs;

	/** Makes the change to the object */
	GEOMETRYFRAMEWORK_API virtual void Apply(UObject* Object) override;

	/** Reverts change to the object */
	GEOMETRYFRAMEWORK_API virtual void Revert(UObject* Object) override;

	/** FRegionMeshChangeBase API -- passes the Vertices array to the ProcessFn lambda */
	GEOMETRYFRAMEWORK_API virtual void ProcessChangeVertices(const UE::Geometry::FDynamicMesh3* ChangedMesh, TFunctionRef<void(TConstArrayView<int32>)> ProcessFn, bool bRevert) const override;

	/** Describes this change (for debugging) */
	GEOMETRYFRAMEWORK_API virtual FString ToString() const override;
};


enum class EMeshVertexChangeComponents : uint8
{
	None = 0,
	VertexPositions = 1,
	VertexColors = 2,
	OverylayUVs = 4,
	OverlayNormals = 16
};
ENUM_CLASS_FLAGS(EMeshVertexChangeComponents);


/**
 * FMeshVertexChangeBuilder can be used to construct a FMeshVertexChange.
 */
class FMeshVertexChangeBuilder
{
public:
	TUniquePtr<FMeshVertexChange> Change;
	TMap<int32, int32> SavedVertices;
	bool bSavePositions = true;
	bool bSaveColors = false;

	bool bSaveOverlayNormals = false;
	TMap<int32, int32> SavedNormalElements;

	bool bSaveOverlayUVs = false;
	TMap<int32, int32> SavedUVElements;

	/** If set, this function is called whenever a newly-seen VertexID is saved, parameters are (VertexID, Index) into saved-vertices array */
	TUniqueFunction<void(int32, int32)> OnNewVertexSaved = nullptr;

	GEOMETRYFRAMEWORK_API FMeshVertexChangeBuilder();
	GEOMETRYFRAMEWORK_API explicit FMeshVertexChangeBuilder(EMeshVertexChangeComponents Components);

	GEOMETRYFRAMEWORK_API void SaveVertexInitial(const FDynamicMesh3* Mesh, int32 VertexID);
	GEOMETRYFRAMEWORK_API void SaveVertexFinal(const FDynamicMesh3* Mesh, int32 VertexID);

	template<typename Enumerable>
	void SaveVertices(const FDynamicMesh3* Mesh, Enumerable Enum, bool bInitial);

	GEOMETRYFRAMEWORK_API void SaveOverlayNormals(const FDynamicMesh3* Mesh, const TArray<int32>& ElementIDs, bool bInitial);
	GEOMETRYFRAMEWORK_API void SaveOverlayNormals(const FDynamicMesh3* Mesh, const TSet<int32>& ElementIDs, bool bInitial);

	GEOMETRYFRAMEWORK_API void SaveOverlayUVs(const FDynamicMesh3* Mesh, const TArray<int32>& ElementIDs, bool bInitial);
	GEOMETRYFRAMEWORK_API void SaveOverlayUVs(const FDynamicMesh3* Mesh, const TSet<int32>& ElementIDs, bool bInitial);

public:
	// currently only used in vertex sculpt tool. cannot be used if bSaveColors = true
	GEOMETRYFRAMEWORK_API void UpdateVertex(int32 VertexID, const FVector3d& OldPosition, const FVector3d& NewPosition);

	// currently only used in element paint tool. Can only be used if bSaveColors=true and bSavePositions=false
	GEOMETRYFRAMEWORK_API void UpdateVertexColor(int32 VertexID, const FVector3f& OldColor, const FVector3f& NewColor);

protected:
	GEOMETRYFRAMEWORK_API void UpdateVertexFinal(int32 VertexID, const FVector3d& NewPosition);

	GEOMETRYFRAMEWORK_API void UpdateOverlayNormal(int32 ElementID, const FVector3f& OldNormal, const FVector3f& NewNormal);
	GEOMETRYFRAMEWORK_API void UpdateOverlayNormalFinal(int32 ElementID, const FVector3f& NewNormal);

	GEOMETRYFRAMEWORK_API void UpdateOverlayUV(int32 ElementID, const FVector2f& OldUV, const FVector2f& NewUV);
	GEOMETRYFRAMEWORK_API void UpdateOverlayUVFinal(int32 ElementID, const FVector2f& NewUV);
};



UINTERFACE(MinimalAPI)
class UMeshVertexCommandChangeTarget : public UInterface
{
	GENERATED_BODY()
};
/**
 * IMeshVertexCommandChangeTarget is an interface which is used to apply a FMeshVertexChange
 */
class IMeshVertexCommandChangeTarget
{
	GENERATED_BODY()
public:
	virtual void ApplyChange(const FMeshVertexChange* Change, bool bRevert) = 0;
};





template<typename Enumerable>
void FMeshVertexChangeBuilder::SaveVertices(const FDynamicMesh3* Mesh, Enumerable Enum, bool bInitial)
{
	if (bInitial)
	{
		for (int32 k : Enum)
		{
			SaveVertexInitial(Mesh, k);
		}
	}
	else
	{
		for (int32 k : Enum)
		{
			SaveVertexFinal(Mesh, k);
		}
	}
}
