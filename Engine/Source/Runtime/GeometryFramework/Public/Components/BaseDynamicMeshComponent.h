// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Components/MeshComponent.h"
#include "InteractiveToolObjects.h"
#include "Changes/MeshVertexChange.h"
#include "Changes/MeshChange.h"
#include "Changes/MeshReplacementChange.h"
#include "MeshConversionOptions.h"
#include "DynamicMesh/DynamicMesh3.h"		// todo replace with predeclaration (lots of fallout)
#include "UDynamicMesh.h"

#include "BaseDynamicMeshComponent.generated.h"

// predecl
struct FMeshDescription;
class FMeshVertexChange;
class FMeshChange;
class FBaseDynamicMeshSceneProxy;
using UE::Geometry::FDynamicMesh3;

/**
 * EMeshRenderAttributeFlags is used to identify different mesh rendering attributes, for things
 * like fast-update functions
 */
enum class EMeshRenderAttributeFlags : uint8
{
	None = 0,
	Positions = 1,
	VertexColors = 1<<1,
	VertexNormals = 1<<2,
	VertexUVs = 1<<3,
	SecondaryIndexBuffers = 1<<4,

	AllVertexAttribs = Positions | VertexColors | VertexNormals | VertexUVs
};
ENUM_CLASS_FLAGS(EMeshRenderAttributeFlags);



/**
 * Tangent calculation modes
 */
UENUM(BlueprintType)
enum class EDynamicMeshComponentTangentsMode : uint8
{
	/** Tangents are not used/available, proceed accordingly (eg generate arbitrary orthogonal basis) */
	NoTangents,
	/** Tangents will be automatically calculated on demand. Note that mesh changes due to tangents calculation will *not* be broadcast via MeshChange events! */
	AutoCalculated,
	/** Tangents are externally provided via the FDynamicMesh3 AttributeSet */
	ExternallyProvided UMETA(DisplayName = "From Dynamic Mesh"),
	/** Tangents mode will be set to the most commonly-useful default -- currently "From Dynamic Mesh" */
	Default = 255
};


/**
 * Color Override Modes
 */
UENUM(BlueprintType)
enum class EDynamicMeshComponentColorOverrideMode : uint8
{
	/** No Color Override enabled */
	None,
	/** Vertex Colors are displayed */
	VertexColors,
	/** Polygroup Colors are displayed */
	Polygroups,
	/** Constant Color is displayed */
	Constant
};


/**
 * Draw Path to use
 */
UENUM(BlueprintType)
enum class EDynamicMeshDrawPath : uint8
{
	/** Use the dynamic draw path, intended for meshes that update on most frames */
	DynamicDraw = 0,
	/** Use the static draw path, for meshes that do not change on most frames */
	StaticDraw = 1
};


/**
 * Mesh Signed Distance Field (SDF) mode
 */
UENUM(BlueprintType)
enum class UE_DEPRECATED(5.0, "Dynamic Mesh distance field support has been deprecated") EDynamicMeshComponentDistanceFieldMode : uint8
{
	/** Do not compute a distance field */
	NoDistanceField = 0,
	/** Compute a distance field in a background thread */
	AsyncCPUDistanceField = 1 UMETA(DisplayName = "Async CPU Distance Field")
};

/**
 * Color Transform to apply to Vertex Colors when converting from internal DynamicMesh
 * Color attributes (eg Color Overlay stored in FVector4f) to RHI Render Buffers (FColor).
 * 
 * Note that UStaticMesh assumes the Source Mesh colors are Linear and always converts to SRGB.
 */
UENUM(BlueprintType)
enum class EDynamicMeshVertexColorTransformMode : uint8
{
	/** Do not apply any color-space transform to Vertex Colors */
	NoTransform = 0,
	/** Assume Vertex Colors are in Linear space and transform to SRGB */
	LinearToSRGB,
	/** Assume Vertex Colors are in SRGB space and convert to Linear */
	SRGBToLinear UMETA(DisplayName = "SRGB To Linear")
};


/**
 * UBaseDynamicMeshComponent is a base interface for a UMeshComponent based on a UDynamicMesh.
 */
UCLASS(Abstract, hidecategories = (LOD), ClassGroup = Rendering, MinimalAPI)
class UBaseDynamicMeshComponent : 
	public UMeshComponent, 
	public IToolFrameworkComponent, 
	public IMeshVertexCommandChangeTarget, 
	public IMeshCommandChangeTarget, 
	public IMeshReplacementCommandChangeTarget
{
	GENERATED_UCLASS_BODY()


	//===============================================================================================================
	// UBaseDynamicMeshComponent API. Subclasses must implement these functions
	//
public:
	/**
	 * initialize the internal mesh from a DynamicMesh
	 */
	virtual void SetMesh(UE::Geometry::FDynamicMesh3&& MoveMesh)
	{
		unimplemented();
	}

	/**
	 * @return pointer to internal mesh
	 * @warning avoid usage of this function, access via GetDynamicMesh() instead
	 */
	virtual FDynamicMesh3* GetMesh()
	{
		unimplemented();
		return nullptr;
	}

	/**
	 * @return pointer to internal mesh
	 */
	virtual const FDynamicMesh3* GetMesh() const
	{
		unimplemented();
		return nullptr;
	}

	/**
	 * Allow external code to read the internal mesh.
	 */
	virtual void ProcessMesh(TFunctionRef<void(const UE::Geometry::FDynamicMesh3&)> ProcessFunc) const
	{
		unimplemented();
	}

	/**
	 * @return the child UDynamicMesh
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	virtual UDynamicMesh* GetDynamicMesh() 
	{ 
		unimplemented();
		return nullptr; 
	}

	/**
	 * Call this if you update the mesh via GetMesh()
	 * @todo should provide a function that calls a lambda to modify the mesh, and only return const mesh pointer
	 */
	virtual void NotifyMeshUpdated()
	{
		unimplemented();
	}

	/**
	 * Apply a vertex deformation change to the internal mesh  (implements IMeshVertexCommandChangeTarget)
	 */
	virtual void ApplyChange(const FMeshVertexChange* Change, bool bRevert) override
	{
		unimplemented();
	}

	/**
	 * Apply a general mesh change to the internal mesh  (implements IMeshCommandChangeTarget)
	 */
	virtual void ApplyChange(const FMeshChange* Change, bool bRevert) override
	{
		unimplemented();
	}

	/**
	* Apply a full mesh replacement change to the internal mesh  (implements IMeshReplacementCommandChangeTarget)
	*/
	virtual void ApplyChange(const FMeshReplacementChange* Change, bool bRevert) override
	{
		unimplemented();
	}

	/**
	 * Apply a transform to the mesh
	 */
	virtual void ApplyTransform(const FTransform3d& Transform, bool bInvert)
	{
		unimplemented();
	}


protected:

	/**
	 * Subclass must implement this to return scene proxy if available, or nullptr
	 */
	virtual FBaseDynamicMeshSceneProxy* GetBaseSceneProxy()
	{
		unimplemented();
		return nullptr;
	}

	/**
	 * Subclass must implement this to notify allocated proxies of updated materials
	 */
	virtual void NotifyMaterialSetUpdated()
	{
		unimplemented();
	}



	//===============================================================================================================
	// Built-in Wireframe-on-Shaded Rendering support. The wireframe looks terrible but this is a convenient 
	// way to enable/disable it.
	//
public:
	/**
	 * If true, render the Wireframe on top of the Shaded Mesh
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dynamic Mesh Component|Rendering", meta = (DisplayName = "Wireframe Overlay") )
	bool bExplicitShowWireframe = false;

	/**
	 * Configure whether wireframe rendering is enabled or not
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	virtual void SetEnableWireframeRenderPass(bool bEnable) { bExplicitShowWireframe = bEnable; }

	/**
	 * @return true if wireframe rendering pass is enabled
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	virtual bool GetEnableWireframeRenderPass() const { return bExplicitShowWireframe; }

	/**
	 * Constant Color used when Override Color Mode is set to Constant
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dynamic Mesh Component|Rendering", meta = (DisplayName = "Wireframe Color"))
	FLinearColor WireframeColor = FLinearColor(0, 0.5f, 1.f);


	//===============================================================================================================
	// Built-in Color Rendering Support. When enabled, Color mode will override any assigned Materials.
	// VertexColor mode displays vertex colors, Polygroup mode displays mesh polygroups via vertex colors,
	// and Constant mode uses ConstantColor as the vertex color. The class-wide DefaultVertexColorMaterial
	// is used as the material that displays the vertex colors, and cannot be overridden per-instance
	// (the OverrideRenderMaterial can be used to do that)
public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dynamic Mesh Component|Rendering", meta = (DisplayName = "Color Override") )
	EDynamicMeshComponentColorOverrideMode ColorMode = EDynamicMeshComponentColorOverrideMode::None;

	/**
	 * Configure the active Color Override
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component|Rendering")
	GEOMETRYFRAMEWORK_API virtual void SetColorOverrideMode(EDynamicMeshComponentColorOverrideMode NewMode);

	/**
	 * @return active Color Override mode
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component|Rendering")
	virtual EDynamicMeshComponentColorOverrideMode GetColorOverrideMode() const { return ColorMode; }

	/**
	 * Constant Color used when Override Color Mode is set to Constant
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dynamic Mesh Component|Rendering", meta = (DisplayName = "Constant Color", EditCondition = "ColorMode==EDynamicMeshComponentColorOverrideMode::Constant"))
	FColor ConstantColor = FColor::White;

	/**
	 * Configure the Color used with Constant Color Override Mode
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component|Rendering")
	GEOMETRYFRAMEWORK_API virtual void SetConstantOverrideColor(FColor NewColor);

	/**
	 * @return active Color used for Constant Color Override Mode
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component|Rendering")
	virtual FColor GetConstantOverrideColor() const { return ConstantColor; }

	/**
	 * Color Space Transform that will be applied to the colors stored in the DynamicMesh Attribute Color Overlay when
	 * constructing render buffers. 
	 * Default is "No Transform", ie color R/G/B/A will be independently converted from 32-bit float to 8-bit by direct mapping.
	 * LinearToSRGB mode will apply SRGB conversion, ie assumes colors in the Mesh are in Linear space. This will produce the same behavior as UStaticMesh.
	 * SRGBToLinear mode will invert SRGB conversion, ie assumes colors in the Mesh are in SRGB space. 
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dynamic Mesh Component|Rendering", meta = (DisplayName = "Vertex Color Space"))
	EDynamicMeshVertexColorTransformMode ColorSpaceMode = EDynamicMeshVertexColorTransformMode::NoTransform;
	
	/**
	 * Configure the active Color Space Transform Mode
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component|Rendering")
	GEOMETRYFRAMEWORK_API virtual void SetVertexColorSpaceTransformMode(EDynamicMeshVertexColorTransformMode NewMode);

	/**
	 * @return active Color Override mode
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component|Rendering")
	virtual EDynamicMeshVertexColorTransformMode GetVertexColorSpaceTransformMode() const { return ColorSpaceMode; }


	//===============================================================================================================
	// Two-Sided rendering support. When enabled, triangles will be rendered whether they are facing the camera or not
public:
	GEOMETRYFRAMEWORK_API virtual void SetTwoSided(bool bEnable);
	virtual bool GetTwoSided() const { return bTwoSided; }

protected:
	UPROPERTY(EditAnywhere, Category = "Dynamic Mesh Component|Rendering");
	bool bTwoSided = false;

	//===============================================================================================================
	// Flat shading support. When enabled, per-triangle normals are computed automatically and used in place
	// of the mesh normals. Mesh tangents are not affected.
public:
	/**
	 * Enable use of per-triangle facet normals in place of mesh normals
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dynamic Mesh Component|Rendering", meta = (DisplayName = "Flat Shading") )
	bool bEnableFlatShading = false;

	/**
	 * Configure the Color used with Constant Color Override Mode
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component|Rendering")
	GEOMETRYFRAMEWORK_API virtual void SetEnableFlatShading(bool bEnable);

	/**
	 * @return Whether mesh will be rendered with per-triangle normals
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component|Rendering")
	virtual bool GetFlatShadingEnabled() const { return bEnableFlatShading; }



	//===============================================================================================================
	// API for changing Rendering settings. Although some of these settings are available publicly
	// on the Component (in some cases as public members), generally changing them requires more complex 
	// Rendering invalidation.
	//
public:

	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void SetShadowsEnabled(bool bEnabled);

	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	virtual bool GetShadowsEnabled() const { return CastShadow; }

	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void SetViewModeOverridesEnabled(bool bEnabled);

	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	virtual bool GetViewModeOverridesEnabled() const { return bEnableViewModeOverrides; }

public:
	/** 
	 * This flag controls whether Editor View Mode Overrides are enabled for this mesh. For example, this controls hidden-line removal on the wireframe 
	 * in Wireframe View Mode, and whether the normal map will be disabled in Lighting-Only View Mode, as well as various other things.
	 * Use SetViewModeOverridesEnabled() to control this setting in Blueprints/C++.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dynamic Mesh Component|Rendering", meta = (DisplayName = "View Mode Overrides") )
	bool bEnableViewModeOverrides = true;


	//===============================================================================================================
	// Override rendering material support. If an Override material is set, then it
	// will be used during drawing of all mesh buffers except Secondary buffers.
	//
public:
	/**
	 * Set an active override render material. This should replace all materials during rendering.
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void SetOverrideRenderMaterial(UMaterialInterface* Material);

	/**
	 * Clear any active override render material
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void ClearOverrideRenderMaterial();
	
	/**
	 * @return true if an override render material is currently enabled for the given MaterialIndex
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	virtual bool HasOverrideRenderMaterial(int k) const
	{
		return OverrideRenderMaterial != nullptr;
	}

	/**
	 * @return active override render material for the given MaterialIndex
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	virtual UMaterialInterface* GetOverrideRenderMaterial(int MaterialIndex) const
	{
		return OverrideRenderMaterial;
	}

protected:
	UPROPERTY()
	TObjectPtr<UMaterialInterface> OverrideRenderMaterial = nullptr;





	//===============================================================================================================
	// Secondary Render Buffers support. This requires implementation in subclasses. It allows
	// a subset of the mesh triangles to be moved to a separate set of render buffers, which
	// can then have a separate material (eg to highlight faces), or be shown/hidden independently.
	// 
public:
	/**
	 * Set an active secondary render material. 
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void SetSecondaryRenderMaterial(UMaterialInterface* Material);

	/**
	 * Clear any active secondary render material
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void ClearSecondaryRenderMaterial();

	/**
	 * @return true if a secondary render material is set
	 */
	virtual bool HasSecondaryRenderMaterial() const
	{
		return SecondaryRenderMaterial != nullptr;
	}

	/**
	 * @return active secondary render material
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	virtual UMaterialInterface* GetSecondaryRenderMaterial() const
	{
		return SecondaryRenderMaterial;
	}

	/**
	 * Show/Hide the secondary triangle buffers. Does not invalidate SceneProxy.
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void SetSecondaryBuffersVisibility(bool bSetVisible);

	/**
	 * @return true if secondary buffers are currently set to be visible
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual bool GetSecondaryBuffersVisibility() const;

protected:
	UPROPERTY()
	TObjectPtr<UMaterialInterface> SecondaryRenderMaterial = nullptr;

	bool bDrawSecondaryBuffers = true;


	//===============================================================================================================
	// Wireframe Material Override Support : Allow to override default wireframe material
	// 
public:
	/**
	 * Set an active wireframe override render Material 
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void SetOverrideWireframeRenderMaterial(UMaterialInterface* Material);

	/**
	 * Clear any active wireframe override render material
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void ClearOverrideWireframeRenderMaterial();

	/**
	 * @return true if a active wireframe override render material is set
	 */
	virtual bool HasOverrideWireframeRenderMaterial() const
	{
		return WireframeMaterialOverride != nullptr;
	}

	/**
	 * @return active wireframe override render material
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	virtual UMaterialInterface* GetOverrideWireframeRenderMaterial() const
	{
		return WireframeMaterialOverride;
	}

protected:
	UPROPERTY();
	TObjectPtr<UMaterialInterface> WireframeMaterialOverride = nullptr;


	//===============================================================================================================
	// Wireframe Material Secondary Override Support : Allow to override default secondary wireframe material
	// 
public:
	/**
	 * Set an active secondary wireframe override render material 
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void SetOverrideSecondaryWireframeRenderMaterial(UMaterialInterface* Material);

	/**
	 * Clear any active secondary wireframe override render material
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void ClearOverrideSecondaryWireframeRenderMaterial();

	/**
	 * @return true if a active secondary wireframe override render material is set
	 */
	virtual bool HasOverrideSecondaryWireframeRenderMaterial() const
	{
		return SecondaryWireframeMaterialOverride != nullptr;
	}

	/**
	 * @return active secondary wireframe override render material
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	virtual UMaterialInterface* GetOverrideSecondaryWireframeRenderMaterial() const
	{
		return SecondaryWireframeMaterialOverride;
	}

protected:
	UPROPERTY();
	TObjectPtr<UMaterialInterface> SecondaryWireframeMaterialOverride = nullptr;


	//===============================================================================================================
	// Raytracing support. Must be enabled for various rendering effects. 
	// However, note that in actual "dynamic" contexts (ie where the mesh is changing every frame),
	// enabling Raytracing support has additional renderthread performance costs and does
	// not currently support partial updates in the SceneProxy.
public:

	/**
	 * Enable/disable Raytracing support on this Mesh, if Raytracing is currently enabled in the Project Settings.
	 * Use SetEnableRaytracing() to configure this flag in Blueprints/C++.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dynamic Mesh Component|Rendering")
	bool bEnableRaytracing = true;

	/**
	 * Enable/Disable raytracing support. This is an expensive call as it flushes
	 * the rendering queue and forces an immediate rebuild of the SceneProxy.
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void SetEnableRaytracing(bool bSetEnabled);

	/**
	 * @return true if raytracing support is currently enabled
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual bool GetEnableRaytracing() const;

protected:
	GEOMETRYFRAMEWORK_API virtual void OnRenderingStateChanged(bool bForceImmedateRebuild);




protected:
	/**
	 * Controls whether to use the dynamic or static draw path
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite,
				BlueprintSetter=SetMeshDrawPath, BlueprintGetter=GetMeshDrawPath,
				 Category = "Dynamic Mesh Component|Rendering")
	EDynamicMeshDrawPath DrawPath = EDynamicMeshDrawPath::DynamicDraw;

	// Whether the fast update paths will be used for the mesh.
	// If false, fast update methods can still be called, but will fall back to recreating the render proxy
	virtual bool AllowFastUpdate()
	{
		// note: fast update is not compatible with static draw path
		return DrawPath == EDynamicMeshDrawPath::DynamicDraw;
	}
	
public:

	/**
	 * Set the mesh rendering to use the static or dynamic draw path
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void SetMeshDrawPath(EDynamicMeshDrawPath NewDrawPath);

	/**
	 * @return Whether the mesh renders using the static or dynamic draw path
	 */
	UFUNCTION(BlueprintPure, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual EDynamicMeshDrawPath GetMeshDrawPath() const;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
protected:
	/** Controls how distance field is computed */
	UE_DEPRECATED(5.6, "Dynamic Mesh distance field support has been deprecated")
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Dynamic Mesh distance field support has been deprecated"))
	EDynamicMeshComponentDistanceFieldMode DistanceFieldMode_DEPRECATED = EDynamicMeshComponentDistanceFieldMode::NoDistanceField;

public:
	/**
	 * Configure Distance Field computation mode
	 */
	UE_DEPRECATED(5.6, "Dynamic Mesh distance field support has been deprecated")
	UFUNCTION(Category = "Dynamic Mesh Component", meta = (DeprecatedFunction, DeprecationMessage = "Dynamic Mesh distance field support has been deprecated"))
	virtual void SetDistanceFieldMode(EDynamicMeshComponentDistanceFieldMode NewDistFieldMode) {}

	/**
	 * @return Method used to compute the distance field
	 */
	UE_DEPRECATED(5.6, "Dynamic Mesh distance field support has been deprecated")
	UFUNCTION(Category = "Dynamic Mesh Component", meta = (DeprecatedFunction, DeprecationMessage = "Dynamic Mesh distance field support has been deprecated"))
	virtual EDynamicMeshComponentDistanceFieldMode GetDistanceFieldMode() const { return EDynamicMeshComponentDistanceFieldMode::NoDistanceField; }
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	
protected:
	// this will be called if a change to distance field mode is detected
	UE_DEPRECATED(5.6, "Dynamic Mesh distance field support has been deprecated")
	GEOMETRYFRAMEWORK_API virtual void OnNewDistanceFieldMode() {}



	//===============================================================================================================
	// Standard Component interfaces
	//
public:

	// UMeshComponent Interface.
	GEOMETRYFRAMEWORK_API virtual int32 GetNumMaterials() const override;
	GEOMETRYFRAMEWORK_API virtual UMaterialInterface* GetMaterial(int32 ElementIndex) const override;
	UE_DEPRECATED(5.7, "Please use GetMaterialRelevance with EShaderPlatform argument and not ERHIFeatureLevel::Type")
	GEOMETRYFRAMEWORK_API virtual FMaterialRelevance GetMaterialRelevance(ERHIFeatureLevel::Type InFeatureLevel) const override;
	GEOMETRYFRAMEWORK_API virtual FMaterialRelevance GetMaterialRelevance(EShaderPlatform InShaderPlatform) const override;
	GEOMETRYFRAMEWORK_API virtual void SetMaterial(int32 ElementIndex, UMaterialInterface* Material) override;
	GEOMETRYFRAMEWORK_API virtual void GetUsedMaterials(TArray<UMaterialInterface*>& OutMaterials, bool bGetDebugMaterials = false) const override;

	GEOMETRYFRAMEWORK_API virtual void SetNumMaterials(int32 NumMaterials);

	//~ Dynamic Mesh component just has an array of materials without managing slot names, but some methods expect
	//~ to access materials via slot name, so we still implement the UPrimitiveComponent interface to do so
	// @return an array of slot names generated from the current materials
	GEOMETRYFRAMEWORK_API virtual TArray<FName> GetMaterialSlotNames() const override;
	// @return true if the dynamic mesh has a material with this slot name
	GEOMETRYFRAMEWORK_API virtual bool IsMaterialSlotNameValid(FName MaterialSlotName) const override;
	// @return a material on this component with this slot name, or null not found
	GEOMETRYFRAMEWORK_API virtual UMaterialInterface* GetMaterialByName(FName MaterialSlotName) const override;

	//~ UObject Interface.
#if WITH_EDITOR
	GEOMETRYFRAMEWORK_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	UPROPERTY()
	TArray<TObjectPtr<UMaterialInterface>> BaseMaterials;





	//===============================================================================================================
	// Class-wide Default Materials used for Wireframe and VertexColor display mode.
	// These are configured for the Editor when the module loads, defaulting to built-in Engine wireframe and
	// vertex color materials. 
	// Note that the ModelingComponents module in the MeshModelingToolset plugin (usually enabled in the UE Editor) 
	// will set a new VertexColor material from that plugins Content. 
	// Client code can further configure these materials as necessary using the static functions below.
public:
	/**
	 * Set the wireframe material used for all BaseDynamicMeshComponent-derived Components
	 */
	static GEOMETRYFRAMEWORK_API void SetDefaultWireframeMaterial(UMaterialInterface* Material);

	/**
	 * Set the vertex color material used for all BaseDynamicMeshComponent-derived Components
	 */
	static GEOMETRYFRAMEWORK_API void SetDefaultVertexColorMaterial(UMaterialInterface* Material);

protected:
	static GEOMETRYFRAMEWORK_API void InitializeDefaultMaterials();
	friend class FGeometryFrameworkModule;			// FGeometryFrameworkModule needs to call the above function

	static GEOMETRYFRAMEWORK_API UMaterialInterface* GetDefaultWireframeMaterial_RenderThread();
	static GEOMETRYFRAMEWORK_API UMaterialInterface* GetDefaultVertexColorMaterial_RenderThread();
	friend class FBaseDynamicMeshSceneProxy;		// FBaseDynamicMeshSceneProxy needs to call these functions...

private:
	// these Materials are used by the render thread. Once the engine is running they should not be modified without 
	// using SetDefaultWireframeMaterial/SetDefaultVertexColorMaterial
	static GEOMETRYFRAMEWORK_API UMaterialInterface* DefaultWireframeMaterial;
	static GEOMETRYFRAMEWORK_API UMaterialInterface* DefaultVertexColorMaterial;
};
