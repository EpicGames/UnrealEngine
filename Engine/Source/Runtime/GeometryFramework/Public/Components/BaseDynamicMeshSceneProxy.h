// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "CoreMinimal.h"
#include "PrimitiveSceneProxy.h"
#include "MeshRenderBufferSet.h"
#include "PrimitiveViewRelevance.h"
#include "DynamicMeshBuilder.h"
#include "Components/BaseDynamicMeshComponent.h"
#include "RayTracingGeometry.h"
#include "Templates/PimplPtr.h"
#include "Util/ProgressCancel.h"
#include "DistanceFieldAtlas.h"

#include "PhysicsEngine/AggregateGeom.h"

using UE::Geometry::FDynamicMesh3;
using UE::Geometry::FDynamicMeshAttributeSet;
using UE::Geometry::FDynamicMeshUVOverlay;
using UE::Geometry::FDynamicMeshNormalOverlay;
using UE::Geometry::FDynamicMeshColorOverlay;
using UE::Geometry::FDynamicMeshMaterialAttribute;

class FDynamicPrimitiveUniformBuffer;
class FMaterialRenderProxy;
class UMaterialInterface;
class FCardRepresentationData;




/**
 * FBaseDynamicMeshSceneProxy is an abstract base class for a Render Proxy
 * for a UBaseDynamicMeshComponent, where the assumption is that mesh data
 * will be stored in FMeshRenderBufferSet instances
 */
class FBaseDynamicMeshSceneProxy : public FPrimitiveSceneProxy
{
	using FIndex2i = UE::Geometry::FIndex2i;
	using FIndex3i = UE::Geometry::FIndex3i;
public:
	UBaseDynamicMeshComponent* ParentBaseComponent;

	/**
	* Utility to initialize and update the mesh render buffers from a mesh with overlays
	* and holds all the settings required.
	*/
	FMeshRenderBufferSetConversionUtil MeshRenderBufferSetConverter;


protected:
	// Set of currently-allocated RenderBuffers. We own these pointers and must clean them up.
	// Must guard access with AllocatedSetsLock!!
	TSet<FMeshRenderBufferSet*> AllocatedBufferSets;

	// use to control access to AllocatedBufferSets 
	FCriticalSection AllocatedSetsLock;

	// control whether the mesh is rendered two-sided
	bool bTwoSided = false;

	// control raytracing support
	bool bEnableRaytracing = false;

	// Allow view-mode overrides. 
	bool bEnableViewModeOverrides = true;

	// whether to try to use the static draw instead of dynamic draw path; note we may still use the dynamic path if collision or vertex color rendering is enabled
	bool bPreferStaticDrawPath = false;

public:
	GEOMETRYFRAMEWORK_API FBaseDynamicMeshSceneProxy(UBaseDynamicMeshComponent* Component);

	GEOMETRYFRAMEWORK_API virtual ~FBaseDynamicMeshSceneProxy();


	//
	// FBaseDynamicMeshSceneProxy API - subclasses must implement these functions
	//


	/**
	 * Return set of active renderbuffers. Must be implemented by subclass.
	 * This is the set of render buffers that will be drawn by GetDynamicMeshElements
	 */
	virtual void GetActiveRenderBufferSets(TArray<FMeshRenderBufferSet*>& Buffers) const = 0;



	//
	// RenderBuffer management
	//


	/**
	 * Allocates a set of render buffers. FPrimitiveSceneProxy will keep track of these
	 * buffers and destroy them on destruction.
	 */
	GEOMETRYFRAMEWORK_API virtual FMeshRenderBufferSet* AllocateNewRenderBufferSet();

	/**
	 * Explicitly release a set of RenderBuffers
	 */
	GEOMETRYFRAMEWORK_API virtual void ReleaseRenderBufferSet(FMeshRenderBufferSet* BufferSet);


	/**
	 * Initialize rendering buffers from given attribute overlays.
	 * Creates three vertices per triangle, IE no shared vertices in buffers.
	 */
	template<typename TriangleEnumerable>
	void InitializeBuffersFromOverlays(
		FMeshRenderBufferSet* RenderBuffers,
		const FDynamicMesh3* Mesh,
		int NumTriangles, const TriangleEnumerable& Enumerable,
		const FDynamicMeshUVOverlay* UVOverlay,
		const FDynamicMeshNormalOverlay* NormalOverlay,
		const FDynamicMeshColorOverlay* ColorOverlay,
		TFunctionRef<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc,
		bool bTrackTriangles = false,
		bool bParallel = false)
	{
		MeshRenderBufferSetConverter.InitializeBuffersFromOverlays(RenderBuffers, Mesh, NumTriangles, Enumerable, UVOverlay, NormalOverlay, ColorOverlay, TangentsFunc, bTrackTriangles, bParallel);
	}



	/**
	 * Initialize rendering buffers from given attribute overlays.
	 * Creates three vertices per triangle, IE no shared vertices in buffers.
	 */
	template<typename TriangleEnumerable, typename UVOverlayListAllocator>
	void InitializeBuffersFromOverlays(
		FMeshRenderBufferSet* RenderBuffers,
		const FDynamicMesh3* Mesh,
		int NumTriangles, const TriangleEnumerable& Enumerable,
		const TArray<const FDynamicMeshUVOverlay*, UVOverlayListAllocator>& UVOverlays,
		const FDynamicMeshNormalOverlay* NormalOverlay,
		const FDynamicMeshColorOverlay* ColorOverlay,
		TFunctionRef<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc,
		bool bTrackTriangles = false,
		bool bParallel = false)
	{
		MeshRenderBufferSetConverter.InitializeBuffersFromOverlays(RenderBuffers, Mesh, NumTriangles, Enumerable, UVOverlays, NormalOverlay, ColorOverlay, TangentsFunc, bTrackTriangles, bParallel);
	}




	/**
	 * Filter the triangles in a FMeshRenderBufferSet into the SecondaryIndexBuffer.
	 * Requires that RenderBuffers->Triangles has been initialized.
	 * @param bDuplicate if set, then primary IndexBuffer is unmodified and SecondaryIndexBuffer contains duplicates. Otherwise triangles are sorted via predicate into either primary or secondary.
	 */
	void UpdateSecondaryTriangleBuffer(
		FMeshRenderBufferSet* RenderBuffers,
		const FDynamicMesh3* Mesh,
		bool bDuplicate)
	{
		MeshRenderBufferSetConverter.UpdateSecondaryTriangleBuffer(RenderBuffers, Mesh, bDuplicate);
	}


	/**
	 * RecomputeRenderBufferTriangleIndexSets re-sorts the existing set of triangles in a FMeshRenderBufferSet
	 * into primary and secondary index buffers. Note that UploadIndexBufferUpdate() must be called
	 * after this function!
	 */
	void RecomputeRenderBufferTriangleIndexSets(
		FMeshRenderBufferSet* RenderBuffers,
		const FDynamicMesh3* Mesh)
	{
		MeshRenderBufferSetConverter.RecomputeRenderBufferTriangleIndexSets(RenderBuffers, Mesh);
	}



	/**
	 * Update vertex positions/normals/colors of an existing set of render buffers.
	 * Assumes that buffers were created with unshared vertices, ie three vertices per triangle, eg by InitializeBuffersFromOverlays()
	 */
	template<typename TriangleEnumerable>
	void UpdateVertexBuffersFromOverlays(
		FMeshRenderBufferSet* RenderBuffers,
		const FDynamicMesh3* Mesh,
		int NumTriangles, const TriangleEnumerable& Enumerable,
		const FDynamicMeshNormalOverlay* NormalOverlay,
		const FDynamicMeshColorOverlay* ColorOverlay,
		TFunctionRef<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc,
		bool bUpdatePositions = true,
		bool bUpdateNormals = false,
		bool bUpdateColors = false)
	{
		MeshRenderBufferSetConverter.UpdateVertexBuffersFromOverlays(RenderBuffers, Mesh, NumTriangles, Enumerable, NormalOverlay, ColorOverlay, TangentsFunc, bUpdatePositions, bUpdateNormals, bUpdateColors);
	}

	/**
	 * Update vertex uvs of an existing set of render buffers.
	 * Assumes that buffers were created with unshared vertices, ie three vertices per triangle, eg by InitializeBuffersFromOverlays()
	 */
	template<typename TriangleEnumerable, typename UVOverlayListAllocator>
	void UpdateVertexUVBufferFromOverlays(
		FMeshRenderBufferSet* RenderBuffers,
		const FDynamicMesh3* Mesh,
		int32 NumTriangles, const TriangleEnumerable& Enumerable,
		const TArray<const FDynamicMeshUVOverlay*, UVOverlayListAllocator>& UVOverlays)
	{
		MeshRenderBufferSetConverter.UpdateVertexUVBufferFromOverlays(RenderBuffers, Mesh, NumTriangles, Enumerable, UVOverlays);
	}


	/**
	 * @return number of active materials
	 */
	GEOMETRYFRAMEWORK_API virtual int32 GetNumMaterials() const;

	/**
	 * Safe GetMaterial function that will never return nullptr
	 */
	GEOMETRYFRAMEWORK_API virtual UMaterialInterface* GetMaterial(int32 k) const;

	/**
	 * Set whether or not to validate mesh batch materials against the component materials.
	 */
	void SetVerifyUsedMaterials(const bool bState)
	{
		bVerifyUsedMaterials = bState;
	}


	/**
	 * This needs to be called if the set of active materials changes, otherwise
	 * the check in FPrimitiveSceneProxy::VerifyUsedMaterial() will fail if an override
	 * material is set, if materials change, etc, etc
	 */
	GEOMETRYFRAMEWORK_API virtual void UpdatedReferencedMaterials();


	//
	// FBaseDynamicMeshSceneProxy implementation
	//

	/**
	 * If EngineShowFlags request vertex color rendering, returns the appropriate vertex color override material's render proxy.  Otherwise returns nullptr.
	 */
	GEOMETRYFRAMEWORK_API static FMaterialRenderProxy* GetEngineVertexColorMaterialProxy(FMeshElementCollector& Collector, const FEngineShowFlags& EngineShowFlags, bool bProxyIsSelected, bool bIsHovered);

	/**
	 * Render set of active RenderBuffers returned by GetActiveRenderBufferSets
	 */
	GEOMETRYFRAMEWORK_API virtual void GetDynamicMeshElements(
		const TArray<const FSceneView*>& Views, 
		const FSceneViewFamily& ViewFamily, 
		uint32 VisibilityMap, 
		FMeshElementCollector& Collector) const override;

protected:
	/**
	 * Helper called by GetDynamicMeshElements to process collision debug drawing
	 */
	GEOMETRYFRAMEWORK_API virtual void GetCollisionDynamicMeshElements(TArray<FMeshRenderBufferSet*>& Buffers,
		const FEngineShowFlags& EngineShowFlags, bool bDrawCollisionView, bool bDrawSimpleCollision, bool bDrawComplexCollision,
		bool bProxyIsSelected,
		const TArray<const FSceneView*>& Views, uint32 VisibilityMap,
		FMeshElementCollector& Collector) const;
public:

	GEOMETRYFRAMEWORK_API virtual void DrawStaticElements(FStaticPrimitiveDrawInterface* PDI) override;

	// Whether to allow use of the static draw path. If false, the dynamic draw path will be used instead.
	GEOMETRYFRAMEWORK_API virtual bool AllowStaticDrawPath(const FSceneView* View) const;

	/**
	 * Draw a single-frame FMeshBatch for a FMeshRenderBufferSet
	 */
	GEOMETRYFRAMEWORK_API virtual void DrawBatch(FMeshElementCollector& Collector,
		const FMeshRenderBufferSet& RenderBuffers,
		const FDynamicMeshIndexBuffer32& IndexBuffer,
		FMaterialRenderProxy* UseMaterial,
		bool bWireframe,
		ESceneDepthPriorityGroup DepthPriority,
		int ViewIndex,
		FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer) const;

	// Like FStaticMeshSceneProxy, only override CreateHitProxies in editor, where component may have CreateMeshHitProxy
#if WITH_EDITOR
	GEOMETRYFRAMEWORK_API virtual HHitProxy* CreateHitProxies(UPrimitiveComponent* Component, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
	GEOMETRYFRAMEWORK_API virtual HHitProxy* CreateHitProxies(IPrimitiveComponent* ComponentInterface, TArray<TRefCountPtr<HHitProxy> >& OutHitProxies) override;
#endif // WITH_EDITOR

	//
	// Raytracing APIs
	//

#if RHI_RAYTRACING

	GEOMETRYFRAMEWORK_API virtual bool IsRayTracingRelevant() const override;
	GEOMETRYFRAMEWORK_API virtual bool HasRayTracingRepresentation() const override;

	GEOMETRYFRAMEWORK_API virtual void GetDynamicRayTracingInstances(FRayTracingInstanceCollector& Collector) override;


	/**
	* Draw a single-frame raytracing FMeshBatch for a FMeshRenderBufferSet
	*/
	GEOMETRYFRAMEWORK_API virtual void DrawRayTracingBatch(
		FRayTracingInstanceCollector& Collector,
		const FMeshRenderBufferSet& RenderBuffers,
		const FDynamicMeshIndexBuffer32& IndexBuffer,
		FRayTracingGeometry& RayTracingGeometry,
		FMaterialRenderProxy* UseMaterialProxy,
		ESceneDepthPriorityGroup DepthPriority,
		FDynamicPrimitiveUniformBuffer& DynamicPrimitiveUniformBuffer) const;


#endif // RHI_RAYTRACING


	//
	// Lumen APIs
	//


public:
	GEOMETRYFRAMEWORK_API virtual const FCardRepresentationData* GetMeshCardRepresentation() const override;

protected:
	TPimplPtr<FCardRepresentationData> MeshCards;

	UE_DEPRECATED(5.6, "Use MeshCards.IsValid() instead")
	bool bMeshCardsValid = false;

	// Helper to set lumen cards
	void UpdateLumenCardsFromBounds();


protected:
	UE_DEPRECATED(5.6, "Distance field support is deprecated for dynamic mesh components")
	TSharedPtr<FDistanceFieldVolumeData> DistanceField;
	UE_DEPRECATED(5.6, "Distance field support is deprecated for dynamic mesh components")
	bool bDistanceFieldValid = false;
public:
	UE_DEPRECATED(5.6, "Distance field support is deprecated for dynamic mesh components")
	static TUniquePtr<FDistanceFieldVolumeData> ComputeDistanceFieldForMesh(
		const FDynamicMesh3& Mesh, 
		FProgressCancel& Progress,
		float DistanceFieldResolutionScale = 1.0, 
		bool bGenerateAsIfTwoSided = false );

	UE_DEPRECATED(5.6, "Distance field support is deprecated for dynamic mesh components")
	GEOMETRYFRAMEWORK_API void SetNewDistanceField(TSharedPtr<FDistanceFieldVolumeData> NewDistanceField, bool bInInitialize);

public:
	// Set the collision data to use for debug drawing, or do nothing if debug drawing is not enabled
	GEOMETRYFRAMEWORK_API void SetCollisionData();

#if UE_ENABLE_DEBUG_DRAWING
private:
	// If debug drawing is enabled, we store collision data here so that collision shapes can be rendered when requested by showflags

	bool bOwnerIsNull = true;
	/** Whether the collision data has been set up for rendering */
	bool bHasCollisionData = false;
	/** Whether a complex collision mesh is available */
	bool bHasComplexMeshData = false;
	/** Collision trace flags */
	ECollisionTraceFlag		CollisionTraceFlag;
	/** Collision Response of this component */
	FCollisionResponseContainer CollisionResponse;
	/** Cached AggGeom holding the collision shapes to render */
	FKAggregateGeom CachedAggGeom;

	// Control access to collision data for debug rendering
	mutable FCriticalSection CachedCollisionLock;

#endif

	GEOMETRYFRAMEWORK_API bool IsCollisionView(const FEngineShowFlags& EngineShowFlags, bool& bDrawSimpleCollision, bool& bDrawComplexCollision) const;

};
