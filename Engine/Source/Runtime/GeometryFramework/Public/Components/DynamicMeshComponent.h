// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Changes/MeshRegionChange.h"
#include "CoreMinimal.h"
#include "BaseDynamicMeshComponent.h"
#include "MeshConversionOptions.h"
#include "Components/MeshRenderDecomposition.h"
#include "DynamicMesh/MeshTangents.h"
#include "TransformTypes.h"
#include "Async/Future.h"
#include "UDynamicMesh.h"
#include "PhysicsEngine/BodySetup.h"

#include "Misc/ScopeLock.h"
#include "Tasks/Task.h"
#include "Util/ProgressCancel.h"
#include "DistanceFieldAtlas.h"

#include "DynamicMeshComponent.generated.h"

// predecl
struct FMeshDescription;

/** internal FPrimitiveSceneProxy defined in DynamicMeshSceneProxy.h */
class FDynamicMeshSceneProxy;
class FBaseDynamicMeshSceneProxy;

/**
 * Interface for a render mesh processor. Use this to process the Mesh stored in UDynamicMeshComponent before
 * sending it off for rendering.
 * NOTE: This is called whenever the Mesh is updated and before rendering, so performance matters.
 */
class IRenderMeshPostProcessor
{
public:
	virtual ~IRenderMeshPostProcessor() = default;

	virtual void ProcessMesh(const FDynamicMesh3& Mesh, FDynamicMesh3& OutRenderMesh) = 0;
};


/** Render data update hint */
UENUM()
enum class EDynamicMeshComponentRenderUpdateMode
{
	/** Do not update render data */
	NoUpdate = 0,
	/** Invalidate overlay of internal component, rebuilding all render data */
	FullUpdate = 1,
	/** Attempt to do partial update of render data if possible */
	FastUpdate = 2
};


template<typename DataType>
struct TAsyncComponentDataComputeQueue
{
	std::atomic<int> JobCounter = 0;
	bool bIsShuttingDown = false;

	~TAsyncComponentDataComputeQueue()
	{
		WaitForAllJobsDuringShutdown();
	}

	struct FComputeJob
	{
		UE::Tasks::FTask Task;
		int JobTimestamp = 0;
		TUniquePtr<FProgressCancel> Progress;
		bool bCancelled = false;
		bool bHasCompleted = false;
	};

	TArray<TUniquePtr<FComputeJob>> PendingJobs;
	FCriticalSection PendingJobsLock;

	// This function will be called w/ the data computed by a job
	// when it finishes, if it is still valid
	// Note this method will be run from a background thread, and should be thread-safe!
	TFunction<void(TUniquePtr<DataType> NewData)> OnComputeCompleted;

	void LaunchJob(const TCHAR* DebugName, TFunction<TUniquePtr<DataType>(FProgressCancel& Progress)> JobWork)
	{
		// OnComputeCompleted function must be set to something
		check(OnComputeCompleted);

		if (!ensure(bIsShuttingDown == false))
		{
			return;
		}

		JobCounter++;
		int CurrentTimestamp = JobCounter;

		// cancel any existing jobs and clear them out if they have returned
		{
			FScopeLock RemovePending(&PendingJobsLock);
			for (int32 k = 0; k < PendingJobs.Num(); ++k)
			{
				PendingJobs[k]->bCancelled = true;
				if (PendingJobs[k]->bHasCompleted)
				{
					PendingJobs.RemoveAtSwap(k, 1, EAllowShrinking::No);
					k--;		// reconsider element that was just swapped in to this position
				}
			}
		}

		// set up the new job
		TUniquePtr<FComputeJob> NewJob = MakeUnique<FComputeJob>();
		FComputeJob* JobPtr = NewJob.Get();
		NewJob->Progress = MakeUnique<FProgressCancel>();
		FProgressCancel* ProgressPtr = NewJob->Progress.Get();
		NewJob->Progress->CancelF = [this, JobPtr]() { return bIsShuttingDown || JobPtr->bCancelled; };
		NewJob->JobTimestamp = CurrentTimestamp;

		// launch it
		NewJob->Task = UE::Tasks::Launch(DebugName, 
			[this, JobWork, JobPtr]() {
				// TODO: limit the number of in-progress active jobs
				TUniquePtr<DataType> Result = JobWork( *JobPtr->Progress );
				if (JobPtr->JobTimestamp == this->JobCounter && JobPtr->bCancelled == false)
				{
					if (!bIsShuttingDown && this->OnComputeCompleted)
					{
						this->OnComputeCompleted(MoveTemp(Result));
					}
				}
				JobPtr->bHasCompleted = true;
			},
			LowLevelTasks::ETaskPriority::BackgroundNormal, 
			UE::Tasks::EExtendedTaskPriority::None );

		// add new job
		{
			FScopeLock AddJob(&PendingJobsLock);
			PendingJobs.Add(MoveTemp(NewJob));
		}
	}


	void WaitForAllJobsDuringShutdown()
	{
		bIsShuttingDown = true;
		FScopeLock PendingLock(&PendingJobsLock);
		for (int32 k = 0; k < PendingJobs.Num(); ++k)
		{
			UE::Tasks::Wait({PendingJobs[k]->Task});
		}
	}

};



/** 
 * UDynamicMeshComponent is a mesh component similar to UProceduralMeshComponent,
 * except it bases the renderable geometry off an internal UDynamicMesh instance (which
 * encapsulates a FDynamicMesh3). 
 * 
 * There is extensive support for partial updates to render buffers, customizing colors,
 * internally decomposing the mesh into separate chunks for more efficient render updates,
 * and support for attaching a 'Postprocessor' to generate a render mesh on-the-fly
 * See comment sections below for details.
 * 
 */
UCLASS(hidecategories = (LOD), meta = (BlueprintSpawnableComponent), ClassGroup = Rendering, MinimalAPI)
class UDynamicMeshComponent : public UBaseDynamicMeshComponent, public IInterface_CollisionDataProvider
{
	GENERATED_UCLASS_BODY()



	//===============================================================================================================
	// Mesh Access. Usage via GetDynamicMesh() or SetMesh()/ProcessMesh()/EditMesh() is preferred, the GetMesh() 
	// pointer access exist largely to support existing code from before UDynamicMesh was added.
public:
	/**
	 * @return pointer to internal mesh
	 * @warning avoid usage of this function, access via GetDynamicMesh() instead
	 */
	virtual FDynamicMesh3* GetMesh() override { return MeshObject ? MeshObject->GetMeshPtr() : nullptr; }

	/**
	 * @return pointer to internal mesh
	 * @warning avoid usage of this function, access via GetDynamicMesh() instead
	 */
	virtual const FDynamicMesh3* GetMesh() const override { return MeshObject ? MeshObject->GetMeshPtr() : nullptr; }

	/**
	 * @return the child UDynamicMesh
	 */
	//UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	virtual UDynamicMesh* GetDynamicMesh() override { return MeshObject; }

	/**
	 * Replace the current UDynamicMesh with a new one, and transfer ownership of NewMesh to this Component.
	 * This can be used to (eg) assign a UDynamicMesh created with NewObject in the Transient Package to this Component.
	 * @warning If NewMesh is owned/Outer'd to another DynamicMeshComponent, a GLEO error may occur if that Component is serialized.
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API void SetDynamicMesh(UDynamicMesh* NewMesh);

	/**
	 * initialize the internal mesh from a DynamicMesh
	 */
	GEOMETRYFRAMEWORK_API virtual void SetMesh(UE::Geometry::FDynamicMesh3&& MoveMesh) override;

	/**
	 * Allow external code to read the internal mesh.
	 */
	GEOMETRYFRAMEWORK_API virtual void ProcessMesh(TFunctionRef<void(const UE::Geometry::FDynamicMesh3&)> ProcessFunc) const;

	/**
	 * Allow external code to to edit the internal mesh.
	 */
	GEOMETRYFRAMEWORK_API virtual void EditMesh(TFunctionRef<void(UE::Geometry::FDynamicMesh3&)> EditFunc,
						  EDynamicMeshComponentRenderUpdateMode UpdateMode = EDynamicMeshComponentRenderUpdateMode::FullUpdate,
						  EDynamicMeshChangeType ChangeType = EDynamicMeshChangeType::GeneralEdit,
						  EDynamicMeshAttributeChangeFlags ChangeFlags = EDynamicMeshAttributeChangeFlags::Unknown);

	/**
	 * Apply transform to internal mesh. In some cases this can be more efficient than a general edit.
	 * @param bInvert if true, inverse tranform is applied instead of forward transform
	 */
	GEOMETRYFRAMEWORK_API virtual void ApplyTransform(const FTransform3d& Transform, bool bInvert) override;


	/**
	 * If set to false, this blocks external code from being able to modify the internal mesh. It prevents modifications through methods on this UDynamicMeshComponent such as EditMesh/SetMesh,
	 * However, this cannot prevent code from modifying the underlying UDynamicMesh through direct references (ex. via GetDynamicMesh()->SetMesh).
	 */
	bool IsEditable() const { return bIsEditable; }
	void SetIsEditable(bool bInIsEditable) { bIsEditable = bInIsEditable; }


	/**
	 * @return Whether geometry elements (triangles, edges, etc) of the dynamic mesh can be interactively selected.
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	bool AllowsGeometrySelection() const { return bAllowsGeometrySelection; }
	/**
	 * Enable/Disable interactive geometry element selection for the mesh. Useful to disable geometry selection on procedural meshes where the selection will be frequently invalidated.
	 * 
	 * @param bInAllowsGeometrySelection Whether geometry elements (triangles, edges, etc) of the dynamic mesh will be interactively selectable.
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	void SetAllowsGeometrySelection(bool bInAllowsGeometrySelection) { bAllowsGeometrySelection = bInAllowsGeometrySelection; }

protected:
	/**
	 * Internal FDynamicMesh is stored inside a UDynamicMesh container, which allows it to be
	 * used from BP, shared with other UObjects, and so on
	 */
	UPROPERTY(Instanced)
	TObjectPtr<UDynamicMesh> MeshObject;




	//===============================================================================================================
	// RenderBuffer Update API. These functions can be used by external code (and internally in some places)
	// to tell the Component that the Mesh data has been modified in some way, and that the RenderBuffers in the RenderProxy
	// need to be updated (or rebuilt entirely). On large meshes a full rebuild is expensive, so there are quite a few
	// variants that can be used to minimize the amount of data updated in different situations.
	//
public:
	/**
	 * Call this if you update the mesh via GetMesh(). This will destroy the existing RenderProxy and create a new one.
	 * @todo should provide a function that calls a lambda to modify the mesh, and only return const mesh pointer
	 */
	GEOMETRYFRAMEWORK_API virtual void NotifyMeshUpdated() override;

	/**
	 * Call this instead of NotifyMeshUpdated() if you have only updated the vertex colors (or triangle color function).
	 * This function will update the existing RenderProxy buffers if possible
	 */
	GEOMETRYFRAMEWORK_API void FastNotifyColorsUpdated();

	/**
	 * Call this instead of NotifyMeshUpdated() if you have only updated the vertex positions (and possibly some attributes).
	 * This function will update the existing RenderProxy buffers if possible
	 */
	GEOMETRYFRAMEWORK_API void FastNotifyPositionsUpdated(bool bNormals = false, bool bColors = false, bool bUVs = false);

	/**
	 * Call this instead of NotifyMeshUpdated() if you have only updated the vertex attributes (but not positions).
	 * This function will update the existing RenderProxy buffers if possible, rather than create new ones.
	 */
	GEOMETRYFRAMEWORK_API void FastNotifyVertexAttributesUpdated(bool bNormals, bool bColors, bool bUVs);

	/**
	 * Call this instead of NotifyMeshUpdated() if you have only updated the vertex positions/attributes
	 * This function will update the existing RenderProxy buffers if possible, rather than create new ones.
	 */
	GEOMETRYFRAMEWORK_API void FastNotifyVertexAttributesUpdated(EMeshRenderAttributeFlags UpdatedAttributes);

	/**
	 * Call this instead of NotifyMeshUpdated() if you have only updated the vertex uvs.
	 * This function will update the existing RenderProxy buffers if possible
	 */
	GEOMETRYFRAMEWORK_API void FastNotifyUVsUpdated();

	/**
	 * Call this instead of NotifyMeshUpdated() if you have only updated secondary triangle sorting.
	 * This function will update the existing buffers if possible, without rebuilding entire RenderProxy.
	 */
	GEOMETRYFRAMEWORK_API void FastNotifySecondaryTrianglesChanged();

	/**
	 * This function updates vertex positions/attributes of existing SceneProxy render buffers if possible, for the given triangles.
	 * If a FMeshRenderDecomposition has not been explicitly set, call is forwarded to FastNotifyVertexAttributesUpdated()
	 */
	GEOMETRYFRAMEWORK_API void FastNotifyTriangleVerticesUpdated(const TArray<int32>& Triangles, EMeshRenderAttributeFlags UpdatedAttributes);

	/**
	 * This function updates vertex positions/attributes of existing SceneProxy render buffers if possible, for the given triangles.
	 * If a FMeshRenderDecomposition has not been explicitly set, call is forwarded to FastNotifyVertexAttributesUpdated()
	 */
	GEOMETRYFRAMEWORK_API void FastNotifyTriangleVerticesUpdated(const TSet<int32>& Triangles, EMeshRenderAttributeFlags UpdatedAttributes);

	/**
	 * This function updates vertex positions/attributes of existing SceneProxy render buffers if possible, for the given vertices.
	 * If a FMeshRenderDecomposition has not been explicitly set, call is forwarded to FastNotifyVertexAttributesUpdated()
	 */
	GEOMETRYFRAMEWORK_API void FastNotifyVerticesUpdated(const TArray<int32>& Vertices, EMeshRenderAttributeFlags UpdatedAttributes);

private:
	// Helper to implement common functionality of above FastNotify-region-Updated methods
	void FastNotifyDecompositionSetsUpdated(TFunctionRef<TArray<int32>(void)> GetUpdatedDecompositionIndices, EMeshRenderAttributeFlags UpdatedAttributes);
public:

	/**
	 * If a Decomposition is set on this Component, and everything is currently valid (proxy/etc), precompute the set of
	 * buffers that will be modified, as well as the bounds of the modified region. These are both computed in parallel.
	 * Use FastNotifyTriangleVerticesUpdated_ApplyPrecompute() with the returned future to apply this precomputation.
	 * @return a future that will (eventually) return true if the precompute is OK, and (immediately) false if it is not
	 */
	GEOMETRYFRAMEWORK_API TFuture<bool> FastNotifyTriangleVerticesUpdated_TryPrecompute(const TArray<int32>& Triangles, TArray<int32>& UpdateSetsOut, UE::Geometry::FAxisAlignedBox3d& BoundsOut);

	/**
	 * This function updates vertex positions/attributes of existing SceneProxy render buffers if possible, for the given triangles.
	 * The assumption is that FastNotifyTriangleVerticesUpdated_TryPrecompute() was used to get the Precompute future, this function
	 * will Wait() until it is done and then use the UpdateSets and UpdateSetBounds that were computed (must be the same variables
	 * passed to FastNotifyTriangleVerticesUpdated_TryPrecompute). 
	 * If the Precompute future returns false, then we forward the call to FastNotifyTriangleVerticesUpdated(), which will do more work.
	 */
	GEOMETRYFRAMEWORK_API void FastNotifyTriangleVerticesUpdated_ApplyPrecompute(const TArray<int32>& Triangles, EMeshRenderAttributeFlags UpdatedAttributes,
		TFuture<bool>& Precompute, const TArray<int32>& UpdateSets, const UE::Geometry::FAxisAlignedBox3d& UpdateSetBounds);


	//===============================================================================================================
	// RenderBuffer Update Blueprint API. 
public:

	/**
	 * Notify the Component that it's DynamicMesh has been modified externally. This will result in all Rendering Data
	 * for the Component being rebuilt on the next frame (internally the Scene Proxy is fully destroyed and rebuilt).
	 *
	 * You must use this function if the mesh triangulation has been modified, or if polygroups or material assignments
	 * have been changed, or if Normal/UV/Color topology has changed (ie new split-vertices have been introduced).
	 * If only vertex attribute values (position, normals, UVs, colors) have been modified, then 
	 * Notify Vertex Attributes Updated can be used to do a faster update.
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component|Rendering", DisplayName = "Notify Mesh Updated")
	GEOMETRYFRAMEWORK_API virtual void NotifyMeshModified();

	/**
	 * Notify the Component that vertex attribute values of it's DynamicMesh have been modified externally. This will result in
	 * Rendering vertex buffers being updated. This update path is more efficient than doing a full Notify Mesh Updated.
	 * 
	 * @warning it is invalid to call this function if (1) the mesh triangulation has also been changed, (2) triangle MaterialIDs have been changed,
	 * or (3) any attribute overlay (normal, color, UV) topology has been modified, ie split-vertices have been added/removed.
	 * Behavior of this function is undefined in these cases and may crash. If you are unsure, use Notify Mesh Updated.
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component|Rendering", DisplayName = "Notify Vertex Attributes Updated")
	GEOMETRYFRAMEWORK_API virtual void NotifyMeshVertexAttributesModified(
		bool bPositions = true, 
		bool bNormals = true,
		bool bUVs = true,
		bool bColors = true);

	//===============================================================================================================
	// Change Support. These changes are primarily used for Undo/Redo, however there is no strict assumption
	// about this internally, objects of these change types could also be used to perform more structured editing.
	// (Note that these functions simply forward the change events to the child UDynamicMesh, which will
	// 	post a mesh-change event that 
	//
public:
	/**
	 * Apply a vertex deformation change to the mesh
	 */
	GEOMETRYFRAMEWORK_API virtual void ApplyChange(const FMeshVertexChange* Change, bool bRevert) override;

	/**
	 * Apply a general mesh change to the mesh
	 */
	GEOMETRYFRAMEWORK_API virtual void ApplyChange(const FMeshChange* Change, bool bRevert) override;

	/**
	* Apply a mesh replacement change to mesh
	*/
	GEOMETRYFRAMEWORK_API virtual void ApplyChange(const FMeshReplacementChange* Change, bool bRevert) override;

	/**
	 * This delegate fires when the mesh has been changed
	 */
	FSimpleMulticastDelegate OnMeshChanged;

	/**
	 * This delegate fires when the mesh vertices have been changed via an FMeshVertexChange
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FMeshVerticesModified, UDynamicMeshComponent*, const FMeshVertexChange*, bool);
	FMeshVerticesModified OnMeshVerticesChanged;

	/**
	 * This delegate fires when the mesh has been changed via an FMeshRegionChange-derived change -- FMeshChange, FMeshVertexChange, or FMeshReplacementChange
	 * Note that in the case of an FMeshVertexChange, the OnMeshVerticesChanged delegate will also fire
	 */
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FMeshRegionModified, UDynamicMeshComponent*, const FMeshRegionChangeBase*, bool);
	FMeshRegionModified OnMeshRegionChanged;

	/**
	 * When a FMeshChange or FMeshVertexChange is applied, by default we currently fully invalidate the render proxy. However in certain
	 * realtime situations (eg like Sculpting tools) it can be critical to undo/redo performance to do more optimized render data updates
	 * (eg using one of the FastXYZ functions above). To allow for that, the full proxy invalidation on change can be (temporarily!) disabled 
	 * using this function. 
	 */
	GEOMETRYFRAMEWORK_API void SetInvalidateProxyOnChangeEnabled(bool bEnabled);

	/** @return true if InvalidateProxyOnChange is enabled (default) */
	bool GetInvalidateProxyOnChangeEnabled() const { return bInvalidateProxyOnChange; }

protected:
	/** If false, we don't completely invalidate the RenderProxy when ApplyChange() is called (assumption is it will be handled elsewhere) */
	bool bInvalidateProxyOnChange = true;

	/** Handle for OnMeshObjectChanged which is registered with MeshObject::OnMeshChanged delegate */
	FDelegateHandle MeshObjectChangedHandle;

	/** Called whenever internal MeshObject is modified, fires OnMeshChanged and OnMeshVerticesChanged above */
	GEOMETRYFRAMEWORK_API void OnMeshObjectChanged(UDynamicMesh* ChangedMeshObject, FDynamicMeshChangeInfo ChangeInfo);

	/**
	 * If set to false, this blocks external code from being able to modify the internal mesh. It prevents modifications through methods on this UDynamicMeshComponent such as EditMesh/SetMesh,
	 * However, this cannot prevent code from modifying the underlying UDynamicMesh through direct references (ex. via GetDynamicMesh()->SetMesh).
	 */
	bool bIsEditable = true;

private:
	/**
	 * If set to false, the geometry element selection UI will not be used for this mesh component. This is useful for procedural meshes where selection would be frequently invalidated.
	 */
	UPROPERTY()
	bool bAllowsGeometrySelection = true;



	//===============================================================================================================
	// Support for specifying per-triangle colors as vertex colors. This allows external code to dynamically override 
	// the vertex colors on the rendered mesh. The lambda that is passed is held for the lifetime of the Component and
	// must remain valid. A Material that uses the vertex colors must be applied, otherwise setting this override will
	// have no visible effect. If the colors change externally, FastNotifyColorsUpdated() can be used to do the 
	// minimal vertex buffer updates necessary in the RenderProxy
	//
public:
	/** Set an active triangle color function if one exists, and update the mesh */
	GEOMETRYFRAMEWORK_API virtual void SetTriangleColorFunction(TUniqueFunction<FColor(const FDynamicMesh3*, int)> TriangleColorFuncIn,
										  EDynamicMeshComponentRenderUpdateMode UpdateMode = EDynamicMeshComponentRenderUpdateMode::FastUpdate);

	/** Clear an active triangle color function if one exists, and update the mesh */
	GEOMETRYFRAMEWORK_API virtual void ClearTriangleColorFunction(EDynamicMeshComponentRenderUpdateMode UpdateMode = EDynamicMeshComponentRenderUpdateMode::FastUpdate);

	/** @return true if a triangle color function is configured */
	GEOMETRYFRAMEWORK_API virtual bool HasTriangleColorFunction();

protected:
	/** If this function is set, we will use these colors instead of vertex colors */
	TUniqueFunction<FColor(const FDynamicMesh3*, int)> TriangleColorFunc = nullptr;

	/** This function is passed via lambda to the RenderProxy to be able to access TriangleColorFunc */
	GEOMETRYFRAMEWORK_API FColor GetTriangleColor(const FDynamicMesh3* Mesh, int TriangleID);

	/** This function is passed via lambda to the RenderProxy when BaseDynamicMeshComponent::ColorMode == Polygroups */
	GEOMETRYFRAMEWORK_API FColor GetGroupColor(const FDynamicMesh3* Mesh, int TriangleID) const;


	//===============================================================================================================
	// Support for Vertex Color remapping/filtering. This allows external code to modulate the existing 
	// Vertex Colors on the rendered mesh. The remapping is only applied to FVector4f Color Overlay attribute buffers.
	// The lambda that is passed is held for the lifetime of the Component and
	// must remain valid. If the Vertex Colors are modified, FastNotifyColorsUpdated() can be used to do the 
	// minimal vertex buffer updates necessary in the RenderProxy
public:
	/** Set an active VertexColor Remapping function if one exists, and update the mesh */
	GEOMETRYFRAMEWORK_API virtual void SetVertexColorRemappingFunction(TUniqueFunction<void(FVector4f&)> ColorMapFuncIn,
		EDynamicMeshComponentRenderUpdateMode UpdateMode = EDynamicMeshComponentRenderUpdateMode::FastUpdate);

	/** Clear an active VertexColor Remapping function if one exists, and update the mesh */
	GEOMETRYFRAMEWORK_API virtual void ClearVertexColorRemappingFunction(EDynamicMeshComponentRenderUpdateMode UpdateMode = EDynamicMeshComponentRenderUpdateMode::FastUpdate);

	/** @return true if a VertexColor Remapping function is configured */
	GEOMETRYFRAMEWORK_API virtual bool HasVertexColorRemappingFunction();

protected:
	/** If this function is set, DynamicMesh Attribute Color Overlay colors will be passed through this function before sending to render buffers */
	TUniqueFunction<void(FVector4f&)> VertexColorMappingFunc = nullptr;

	/** This function is passed via lambda to the RenderProxy to be able to access VertexColorMappingFunc */
	GEOMETRYFRAMEWORK_API void RemapVertexColor(FVector4f& VertexColorInOut);



	//===============================================================================================================
	// Support for Secondary triangle index buffers. When this is configured, then triangles identified
	// by the filtering predicate function will be placed in a second set of RenderBuffers at the SceneProxy level.
	// This can be combined with the SecondaryRenderMaterial support in UBaseDynamicMeshComponent to draw
	// that triangle set with a different material, to efficiently accomplish UI features like highlighting a
	// subset of mesh triangles.
	//
public:
	/**
	 * If Secondary triangle buffers are enabled, then we will filter triangles that pass the given predicate
	 * function into a second index buffer. These triangles will be drawn with the Secondary render material
	 * that is set in the BaseDynamicMeshComponent. Calling this function invalidates the SceneProxy.
	 */
	GEOMETRYFRAMEWORK_API virtual void EnableSecondaryTriangleBuffers(TUniqueFunction<bool(const FDynamicMesh3*, int32)> SecondaryTriFilterFunc);

	/**
	 * Disable secondary triangle buffers. This invalidates the SceneProxy.
	 */
	GEOMETRYFRAMEWORK_API virtual void DisableSecondaryTriangleBuffers();

protected:
	TUniqueFunction<bool(const FDynamicMesh3*, int32)> SecondaryTriFilterFunc = nullptr;




	//===============================================================================================================
	// Support for a Render Decomposition, which is basically a segmentation of the mesh triangles into
	// subsets which will be turned into separate RenderBuffers in the Render Proxy. If this is configured,
	// then various of the FastNotifyXYZUpdated() functions above will only need to rebuild the RenderBuffers
	// that include affected triangles. The FMeshRenderDecomposition implementation has various options for
	// building decompositions based on material, spatial clustering, etc.
	//
public:
	/**
	 * Configure a decomposition of the mesh, which will result in separate render buffers for each 
	 * decomposition triangle group. Invalidates existing SceneProxy.
	 */
	GEOMETRYFRAMEWORK_API virtual void SetExternalDecomposition(TUniquePtr<FMeshRenderDecomposition> Decomposition);

protected:
	TUniquePtr<FMeshRenderDecomposition> Decomposition;




	//===============================================================================================================
	// IRenderMeshPostProcessor Support. If a RenderMesh Postprocessor is configured, then instead of directly 
	// passing the internal mesh to the RenderProxy, IRenderMeshPostProcessor::PostProcess is applied to populate
	// the internal RenderMesh which is passed instead. This allows things like Displacement or Subdivision to be 
	// done on-the-fly at the rendering level (which is potentially more efficient).
	//
public:
	/**
	 * Add a render mesh processor, to be called before the mesh is sent for rendering.
	 */
	GEOMETRYFRAMEWORK_API virtual void SetRenderMeshPostProcessor(TUniquePtr<IRenderMeshPostProcessor> Processor);

	/**
	 * The SceneProxy should call these functions to get the post-processed RenderMesh. (See IRenderMeshPostProcessor.)
	 */
	GEOMETRYFRAMEWORK_API virtual FDynamicMesh3* GetRenderMesh();

	/**
	 * The SceneProxy should call these functions to get the post-processed RenderMesh. (See IRenderMeshPostProcessor.)
	 */
	GEOMETRYFRAMEWORK_API virtual const FDynamicMesh3* GetRenderMesh() const;

protected:
	TUniquePtr<IRenderMeshPostProcessor> RenderMeshPostProcessor;
	TUniquePtr<FDynamicMesh3> RenderMesh;




	//===============================================================================================================
	// Support for Component attachment change notifications via delegates. Standard UE
	// Actor/Component hierarchy does not generally provide these capabilities, but in some use
	// cases (eg procedural mesh Actors) we need to know things like when the Component set inside
	// an Actor is modified.
public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FComponentChildrenChangedDelegate, USceneComponent*, bool);

	/**
	 * The OnChildAttached() and OnChildDetached() implementations (from USceneComponent API) broadcast this delegate. This
	 * allows Actors that have UDynamicMeshComponent's to respond to changes in their Component hierarchy.
	 */
	FComponentChildrenChangedDelegate OnChildAttachmentModified;



	//===============================================================================================================
	// Material Set API. DynamicMeshComponent supports changing the Material Set dynamically, even at Runtime.
public:

	/**
	 * Set new list of Materials for the Mesh. Dynamic Mesh Component does not have 
	 * Slot Names, so the size of the Material Set should be the same as the number of
	 * different Material IDs on the mesh MaterialID attribute
	 * @param bDeleteExtraSlots if true, extra Material Slots beyond max NewMaterialSet.Num() are removed
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API void ConfigureMaterialSet(const TArray<UMaterialInterface*>& NewMaterialSet, bool bDeleteExtraSlots = true);

	/**
	 * Compute the maximum MaterialID on the DynamicMesh, and ensure that Material Slots match.
	 * Pass both arguments as false to just do a check.
	 * @param bCreateIfMissing if true, add extra (empty) Material Slots to match max MaterialID
	 * @param bDeleteExtraSlots if true, extra Material Slots beyond max MaterialID are removed
	 * @return true if at the end of this function, Material Slot Count == Max MaterialID
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API bool ValidateMaterialSlots(bool bCreateIfMissing = true, bool bDeleteExtraSlots = true);


	//===============================================================================================================
	// Triangle-Vertex Tangents support. The default behavior is to use the provided external Tangents.
	// If TangentsType == EDynamicMeshComponentTangentsMode::ExternallyProvided, the Tangent and Bitangent attributes of
	//   the FDynamicMesh3 AttributeSet are used at the SceneProxy level, the Component is not involved
	// If TangentsType == EDynamicMeshComponentTangentsMode::AutoCalculated, the Tangents are computed internally using
	//   a fast MikkT approximation via FMeshTangentsf. They will be recomputed when the mesh is modified, however
	//   they are *not* recomputed when using the Fast Update functions above (in that case InvalidateAutoCalculatedTangents() 
	//   can be used to force recomputation)
	// If TangentsType == EDynamicMeshComponentTangentsMode::NoTangents, the Component will not use Tangents, which will
	//   lead to incorrect rendering for any material with Normal Maps and some other shaders.
	//
private:
	// Default what the 'Default' tangent type should map to
	static constexpr EDynamicMeshComponentTangentsMode UseDefaultTangentsType = EDynamicMeshComponentTangentsMode::ExternallyProvided;

public:
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API void SetTangentsType(EDynamicMeshComponentTangentsMode NewTangentsType);

	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	EDynamicMeshComponentTangentsMode GetTangentsType() const
	{
		return TangentsType == EDynamicMeshComponentTangentsMode::Default ? UseDefaultTangentsType : TangentsType;
	}

private:
	// pure version of GetTangentsType, so it can be used as a getter below (getters must be BlueprintPure)
	UFUNCTION(BlueprintCallable, BlueprintPure, BlueprintInternalUseOnly, Category = "Dynamic Mesh Component")
	EDynamicMeshComponentTangentsMode GetTangentsTypePure() const
	{
		return GetTangentsType();
	}

public:

	/** This function marks the auto tangents as dirty, they will be recomputed before they are used again */
	GEOMETRYFRAMEWORK_API virtual void InvalidateAutoCalculatedTangents();

	/** @return AutoCalculated Tangent Set, which may require that they be recomputed, or nullptr if not enabled/available */
	GEOMETRYFRAMEWORK_API const UE::Geometry::FMeshTangentsf* GetAutoCalculatedTangents();

protected:
	/** Tangent source defines whether we use the provided tangents on the Dynamic Mesh, autogenerate tangents, or do not use tangents */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, BlueprintSetter=SetTangentsType, BlueprintGetter=GetTangentsTypePure, Category = "Dynamic Mesh Component|Rendering")
	EDynamicMeshComponentTangentsMode TangentsType = EDynamicMeshComponentTangentsMode::Default;

	/** true if AutoCalculatedTangents has been computed for current mesh */
	bool bAutoCalculatedTangentsValid = false;

	/** Set of per-triangle-vertex tangents computed for the current mesh. Only valid if bAutoCalculatedTangentsValid == true */
	UE::Geometry::FMeshTangentsf AutoCalculatedTangents;

	GEOMETRYFRAMEWORK_API void UpdateAutoCalculatedTangents();


	//===============================================================================================================
	//
	// Distance Field Support
	//
protected:
	UE_DEPRECATED(5.6, "Dynamic Mesh distance field support has been deprecated")
	FCriticalSection DistanceFieldLock;
	UE_DEPRECATED(5.6, "Dynamic Mesh distance field support has been deprecated")
	TSharedPtr<FDistanceFieldVolumeData> CurrentDistanceField;
	UE_DEPRECATED(5.6, "Dynamic Mesh distance field support has been deprecated")
	virtual void UpdateDistanceField() {}
	UE_DEPRECATED(5.6, "Dynamic Mesh distance field support has been deprecated")
	TAsyncComponentDataComputeQueue<FDistanceFieldVolumeData> DistanceFieldComputeQueue;
	UE_DEPRECATED(5.6, "Dynamic Mesh distance field support has been deprecated")
	virtual void OnNewDistanceFieldData_Async(TUniquePtr<FDistanceFieldVolumeData> NewData) {}



	//===============================================================================================================
	//
	// Physics APIs
	//
public:
	/**
	 * calls SetComplexAsSimpleCollisionEnabled(true, true)
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API void EnableComplexAsSimpleCollision();

	/**
	 * If bEnabled=true, sets bEnableComplexCollision=true and CollisionType=CTF_UseComplexAsSimple
	 * If bEnabled=true, sets bEnableComplexCollision=false and CollisionType=CTF_UseDefault
	 * @param bImmediateUpdate if true, UpdateCollision(true) is called
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API void SetComplexAsSimpleCollisionEnabled(bool bEnabled, bool bImmediateUpdate = true);

	/**
	 * Set value of bDeferCollisionUpdates, when enabled, collision is not automatically recomputed each time the mesh changes.
	 * @param bImmediateUpdate if true, UpdateCollision(true) is called if bEnabled=false, ie to force a collision rebuild
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API void SetDeferredCollisionUpdatesEnabled(bool bEnabled, bool bImmediateUpdate = true);


	GEOMETRYFRAMEWORK_API virtual bool GetPhysicsTriMeshData(struct FTriMeshCollisionData* CollisionData, bool InUseAllTriData) override;
	GEOMETRYFRAMEWORK_API virtual bool GetTriMeshSizeEstimates(struct FTriMeshCollisionDataEstimates& OutTriMeshEstimates, bool bInUseAllTriData) const override;
	GEOMETRYFRAMEWORK_API virtual bool ContainsPhysicsTriMeshData(bool InUseAllTriData) const override;
	GEOMETRYFRAMEWORK_API virtual bool WantsNegXTriMesh() override;

	/** @return current BodySetup for this Component, or nullptr if it does not exist */
	virtual const UBodySetup* GetBodySetup() const { return MeshBodySetup; }
	/** @return BodySetup for this Component. A new BodySetup will be created if one does not exist. */
	GEOMETRYFRAMEWORK_API virtual UBodySetup* GetBodySetup() override;

	/**
	 * Force an update of the Collision/Physics data for this Component.
	 * @param bOnlyIfPending only update if a collision update is pending, ie the underlying DynamicMesh changed and bDeferCollisionUpdates is enabled
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh Component")
	GEOMETRYFRAMEWORK_API virtual void UpdateCollision(bool bOnlyIfPending = true);

	/** Type of Collision Geometry to use for this Mesh */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dynamic Mesh Component|Collision")
	TEnumAsByte<enum ECollisionTraceFlag> CollisionType = ECollisionTraceFlag::CTF_UseSimpleAsComplex;

	/** 
	 * Update the simple collision shapes associated with this mesh component
	 * 
	 * @param AggGeom				New simple collision shapes to be used
	 * @param bUpdateCollision		Whether to automatically call UpdateCollision() -- if false, manually call it to register the change with the physics system
	 */
	GEOMETRYFRAMEWORK_API virtual void SetSimpleCollisionShapes(const struct FKAggregateGeom& AggGeom, bool bUpdateCollision);

	const FKAggregateGeom& GetSimpleCollisionShapes() const
	{
		return AggGeom;
	}

	/**
	 * Clear the simple collision shapes associated with this mesh component
	 * @param bUpdateCollision		Whether to automatically call UpdateCollision() -- if false, manually call it to register the change with the physics system
	 */
	GEOMETRYFRAMEWORK_API virtual void ClearSimpleCollisionShapes(bool bUpdateCollision);

	/**
	 *	Controls whether the physics cooking should be done off the game thread.
	 *  This should be used when collision geometry doesn't have to be immediately up to date (For example streaming in far away objects)
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Dynamic Mesh Component|Collision")
	bool bUseAsyncCooking = false;

	/** 
	 * If true, current mesh will be used as Complex Collision source mesh. 
	 * This is independent of the CollisionType setting, ie, even if Complex collision is enabled, if this is false, then the Complex Collision mesh will be empty
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dynamic Mesh Component|Collision");
	bool bEnableComplexCollision = false;

	/** If true, updates to the mesh will not result in immediate collision regeneration. Useful when the mesh will be modified multiple times before collision is needed. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dynamic Mesh Component|Collision");
	bool bDeferCollisionUpdates = false;

	/** If true, UV hit results will not be enabled for this mesh's complex collision even if they are enabled for the project settings generally. This can make the complex build faster and take less memory. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Dynamic Mesh Component|Collision");
	bool bDisableMeshUVHitResults = false;

	/**
	 * The bDeferCollisionUpdates property is a serialized Component setting that controls when collision is regenerated.
	 * However, in some cases where a UDynamicMesh is being programmatically edited (eg like a live deformation/etc), we
	 * want to defer collision updates during the user interaction, but not change the serialized property. 
	 * In this case a non-serialized version of the flag can be *temporarily* set via this function in C++. 
	 */
	GEOMETRYFRAMEWORK_API void SetTransientDeferCollisionUpdates(bool bEnabled);

protected:
	UPROPERTY(Instanced)
	TObjectPtr<UBodySetup> MeshBodySetup;

	GEOMETRYFRAMEWORK_API virtual void InvalidatePhysicsData();
	GEOMETRYFRAMEWORK_API virtual void RebuildPhysicsData();

	bool bTransientDeferCollisionUpdates = false;
	bool bCollisionUpdatePending = false;

protected:

	//
	// standard Component internals, for computing bounds and managing the SceneProxy
	//

	/** Current local-space bounding box of Mesh */
	UE::Geometry::FAxisAlignedBox3d LocalBounds;

	/** Recompute LocalBounds from the current Mesh */
	GEOMETRYFRAMEWORK_API void UpdateLocalBounds();

	//
	// Internals for managing collision representation and setup
	//

	/** Simplified collision representation for the mesh component  */
	UPROPERTY(EditAnywhere, Category = BodySetup, meta = (DisplayName = "Primitives", NoResetToDefault))
	struct FKAggregateGeom AggGeom;

	/** Queue for async body setups that are being cooked */
	UPROPERTY(transient)
	TArray<TObjectPtr<UBodySetup>> AsyncBodySetupQueue;

	/** Once async physics cook is done, create needed state */
	GEOMETRYFRAMEWORK_API virtual void FinishPhysicsAsyncCook(bool bSuccess, UBodySetup* FinishedBodySetup);

	GEOMETRYFRAMEWORK_API virtual UBodySetup* CreateBodySetupHelper();
	
	/** @return Set new BodySetup for this Component. */
	GEOMETRYFRAMEWORK_API virtual void SetBodySetup(UBodySetup* NewSetup);

	/**
	 * This is called to tell our RenderProxy about modifications to the material set.
	 * We need to pass this on for things like material validation in the Editor.
	 */
	GEOMETRYFRAMEWORK_API virtual void NotifyMaterialSetUpdated();

	/**
	 * If the render proxy is invalidated (eg by MarkRenderStateDirty()), it will be destroyed at the end of
	 * the frame, but the base SceneProxy pointer is not nulled out immediately. As a result if we call various
	 * partial-update functions after invalidating the proxy, they may be operating on an invalid proxy.
	 * So we have to keep track of proxy-valid state ourselves.
	 */
	bool bProxyValid = false;

	/**
	 * If true, the render proxy will verify that the mesh batch materials match the contents from
	 * the component GetUsedMaterials(). Used material verification is prone to races when changing
	 * materials on this component in quick succession (for example, SetOverrideRenderMaterial). This
	 * parameter is provided to allow clients to opt out of used material verification for these
	 * use cases.
	 */
	bool bProxyVerifyUsedMaterials = true;

	virtual FBaseDynamicMeshSceneProxy* GetBaseSceneProxy() override { return (FBaseDynamicMeshSceneProxy*)GetCurrentSceneProxy(); }

	/** 
	 * @return current render proxy, if valid, otherwise nullptr 
	 */
	GEOMETRYFRAMEWORK_API FDynamicMeshSceneProxy* GetCurrentSceneProxy();


	/**
	 * Fully invalidate all rendering data for this Component. Current Proxy will be discarded, Bounds and possibly Tangents recomputed, etc
	 */
	GEOMETRYFRAMEWORK_API void ResetProxy();

	//~ Begin UPrimitiveComponent Interface.
	GEOMETRYFRAMEWORK_API virtual FPrimitiveSceneProxy* CreateSceneProxy() override;

	//~ USceneComponent Interface.
	GEOMETRYFRAMEWORK_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	GEOMETRYFRAMEWORK_API virtual void OnChildAttached(USceneComponent* ChildComponent) override;
	GEOMETRYFRAMEWORK_API virtual void OnChildDetached(USceneComponent* ChildComponent) override;

	//~ UObject Interface.
	GEOMETRYFRAMEWORK_API virtual void Serialize(FArchive& Ar) override;
	GEOMETRYFRAMEWORK_API virtual void PostLoad() override; // called after load
	GEOMETRYFRAMEWORK_API virtual void PostEditImport() override; // called after duplicate/copy
	GEOMETRYFRAMEWORK_API virtual void BeginDestroy() override;
#if WITH_EDITOR
	GEOMETRYFRAMEWORK_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

public:
	/** Set whether or not to validate mesh batch materials against the component materials. */
	GEOMETRYFRAMEWORK_API void SetSceneProxyVerifyUsedMaterials(bool bState);


private:
	// Internal helper to be called when mesh data is updated
	void InternalOnMeshUpdated();

	void BroadcastMeshPropertyChangeEvent();

};
