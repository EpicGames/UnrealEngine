// Copyright Epic Games, Inc. All Rights Reserved. 

#pragma once

#include "CoreMinimal.h"

#include "Async/ParallelFor.h"

#include "Components/BaseDynamicMeshComponent.h"
#include "Components/DynamicMeshComponent.h"
#include "DynamicMeshBuilder.h"
#include "DynamicMesh/DynamicMesh3.h"	
#include "DynamicMesh/DynamicMeshAttributeSet.h"

#include "LocalVertexFactory.h"
#include "Rendering/StaticMeshVertexBuffer.h"
#include "Rendering/PositionVertexBuffer.h"
#include "Rendering/ColorVertexBuffer.h"
#include "RHICommandList.h"
#include "RayTracingGeometry.h"
#include "VertexFactory.h"




using UE::Geometry::FDynamicMesh3;
using UE::Geometry::FDynamicMeshAttributeSet;
using UE::Geometry::FDynamicMeshUVOverlay;
using UE::Geometry::FDynamicMeshNormalOverlay;
using UE::Geometry::FDynamicMeshColorOverlay;
using UE::Geometry::FDynamicMeshMaterialAttribute;

class UMaterialInterface;

/**
 * FMeshRenderBufferSet stores a set of RenderBuffers for a mesh
 */
class FMeshRenderBufferSet
{
public:
	/** Number of triangles in this renderbuffer set. Note that triangles may be split between IndexBuffer and SecondaryIndexBuffer. */
	int TriangleCount = 0;

	/** The buffer containing vertex data. */
	FStaticMeshVertexBuffer StaticMeshVertexBuffer;
	/** The buffer containing the position vertex data. */
	FPositionVertexBuffer PositionVertexBuffer;
	/** The buffer containing the vertex color data. */
	FColorVertexBuffer ColorVertexBuffer;

	/** triangle indices */
	FDynamicMeshIndexBuffer32 IndexBuffer;

	/** vertex factory */
	FLocalVertexFactory VertexFactory;

	/** Material to draw this mesh with */
	UMaterialInterface* Material = nullptr;

	/**
	 * Optional list of triangles stored in this buffer. Storing this allows us
	 * to rebuild the buffers if vertex data changes.
	 */
	TOptional<TArray<int>> Triangles;

	/**
	 * If secondary index buffer is enabled, we populate this index buffer with additional triangles indexing into the same vertex buffers
	 */
	bool bEnableSecondaryIndexBuffer = false;

	/**
	 * partition or subset of IndexBuffer that indexes into same vertex buffers
	 */
	FDynamicMeshIndexBuffer32 SecondaryIndexBuffer;

	/**
	 * configure whether raytracing should be enabled for this RenderBufferSet
	 */
	bool bEnableRaytracing = false;

#if RHI_RAYTRACING
	/**
	 * Raytracing buffers
	 */
	FRayTracingGeometry PrimaryRayTracingGeometry;
	FRayTracingGeometry SecondaryRayTracingGeometry;
	bool bIsRayTracingDataValid = false;
#endif

	/**
	 * In situations where we want to *update* the existing Vertex or Index buffers, we need to synchronize
	 * access between the Game and Render threads. We use this lock to do that.
	 */
	FCriticalSection BuffersLock;


	FMeshRenderBufferSet(ERHIFeatureLevel::Type FeatureLevelType)
	: VertexFactory(FeatureLevelType, "FMeshRenderBufferSet")
	{
		StaticMeshVertexBuffer.SetUseFullPrecisionUVs(true);
		StaticMeshVertexBuffer.SetUseHighPrecisionTangentBasis(true);

#if RHI_RAYTRACING
		EnumAddFlags(IndexBuffer.UsageFlags, EBufferUsageFlags::ShaderResource);
		EnumAddFlags(SecondaryIndexBuffer.UsageFlags, EBufferUsageFlags::ShaderResource);
#endif // RHI_RAYTRACING
	}

	virtual ~FMeshRenderBufferSet()
	{
		if (TriangleCount > 0)
		{
			PositionVertexBuffer.ReleaseResource();
			StaticMeshVertexBuffer.ReleaseResource();
			ColorVertexBuffer.ReleaseResource();
			VertexFactory.ReleaseResource();
			if (IndexBuffer.IsInitialized())
			{
				IndexBuffer.ReleaseResource();
			}
			if (SecondaryIndexBuffer.IsInitialized())
			{
				SecondaryIndexBuffer.ReleaseResource();
			}

#if RHI_RAYTRACING
			if (bEnableRaytracing)
			{
				PrimaryRayTracingGeometry.ReleaseResource();
				SecondaryRayTracingGeometry.ReleaseResource();
			}
#endif
		}
	}


	/**
	 * Upload initialized mesh buffers.
	 * @warning This can only be called on the Rendering Thread.
	 */
	GEOMETRYFRAMEWORK_API void Upload();
	


	/**
	 * Fast path to only update the primary and secondary index buffers. This can be used
	 * when (eg) the secondary index buffer is being used to highlight/hide a subset of triangles.
	 * @warning This can only be called on the Rendering Thread.
	 */
	void UploadIndexBufferUpdate()
	{
		// todo: can this be done with RHI locking and memcpy, like in TransferVertexUpdateToGPU?
		FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

		if (IndexBuffer.Indices.Num() > 0)
		{
			InitOrUpdateResource(RHICmdList, &IndexBuffer);
		}
		if (bEnableSecondaryIndexBuffer && SecondaryIndexBuffer.Indices.Num() > 0)
		{
			InitOrUpdateResource(RHICmdList, &SecondaryIndexBuffer);
		}

		InvalidateRayTracingData();
		ValidateRayTracingData();		// currently we are immediately validating. This may be revisited in future.
	}


	/**
	 * Fast path to only update vertex buffers. This path rebuilds all the
	 * resources and reconfigures the vertex factory, so the counts/etc could be modified.
	 * @warning This can only be called on the Rendering Thread.
	 */
	GEOMETRYFRAMEWORK_API void UploadVertexUpdate(bool bPositions, bool bMeshAttribs, bool bColors);
	


	/**
	 * Fast path to update various vertex buffers. This path does not support changing the
	 * size/counts of any of the sub-buffers, a direct memcopy from the CPU-side buffer to the RHI buffer is used.
	 * @warning This can only be called on the Rendering Thread.
	 */
	GEOMETRYFRAMEWORK_API void TransferVertexUpdateToGPU(FRHICommandListBase& RHICmdList, bool bPositions, bool bNormals, bool bTexCoords, bool bColors);


	void InvalidateRayTracingData()
	{
#if RHI_RAYTRACING
		bIsRayTracingDataValid = false;
#endif
	}

	// Verify that valid raytracing data is available. This will cause a rebuild of the
	// raytracing data if any of our buffers have been modified. Currently this is called
	// by GetDynamicRayTracingInstances to ensure the RT data is available when needed.
	void ValidateRayTracingData()
	{
#if RHI_RAYTRACING
		if (bIsRayTracingDataValid == false && IsRayTracingEnabled() && bEnableRaytracing)
		{
			UpdateRaytracingGeometryIfEnabled();

			bIsRayTracingDataValid = true;
		}
#endif
	}


protected:

	// rebuild raytracing data for current buffers
	void UpdateRaytracingGeometryIfEnabled()
	{
#if RHI_RAYTRACING
		// do we always want to do this?
		PrimaryRayTracingGeometry.ReleaseResource();
		SecondaryRayTracingGeometry.ReleaseResource();
		FRHICommandListBase& RHICmdList = FRHICommandListImmediate::Get();

		for (int32 k = 0; k < 2; ++k)
		{
			FDynamicMeshIndexBuffer32& UseIndexBuffer = (k == 0) ? IndexBuffer : SecondaryIndexBuffer;
			if (UseIndexBuffer.Indices.Num() == 0)
			{
				continue;
			}

			FRayTracingGeometry& RayTracingGeometry = (k == 0) ? PrimaryRayTracingGeometry : SecondaryRayTracingGeometry;

			FRayTracingGeometryInitializer Initializer;
			Initializer.IndexBuffer = UseIndexBuffer.IndexBufferRHI;
			Initializer.TotalPrimitiveCount = UseIndexBuffer.Indices.Num() / 3;
			Initializer.GeometryType = RTGT_Triangles;
			Initializer.bFastBuild = true;
			Initializer.bAllowUpdate = false;

			FRayTracingGeometrySegment Segment;
			Segment.VertexBuffer = PositionVertexBuffer.VertexBufferRHI;
			Segment.NumPrimitives = Initializer.TotalPrimitiveCount;
			Segment.MaxVertices = PositionVertexBuffer.GetNumVertices();

			Initializer.Segments.Add(Segment);

			RayTracingGeometry.SetInitializer(MoveTemp(Initializer));
			RayTracingGeometry.InitResource(RHICmdList);
		}
#endif
	}



	/**
	 * Initializes a render resource, or update it if already initialized.
	 * @warning This function can only be called on the Render Thread
	 */
	void InitOrUpdateResource(FRHICommandListBase& RHICmdList, FRenderResource* Resource)
	{
		if (!Resource->IsInitialized())
		{
			Resource->InitResource(RHICmdList);
		}
		else
		{
			Resource->UpdateRHI(RHICmdList);
		}
	}


public: 

	/**
	 * Enqueue a command on the Render Thread to destroy the passed in buffer set.
	 * At this point the buffer set should be considered invalid.
	 */
	static void DestroyRenderBufferSet(FMeshRenderBufferSet* BufferSet)
	{
		//@todo, remove this check?  based on our usage it could produce a memory leak - see BaseDynamicMeshSceneProxy.cpp
		if (BufferSet->TriangleCount == 0)
		{
			return;
		}

		delete BufferSet;
	}


};


/**
* Parameters and accompanying functions used to build and update FMeshRenderBuffers from a dynamic mesh.
* For usage, see FBaseDynamicMeshSceneProxy or the simple converter below FDynamicMeshComponentToMeshRenderBufferSet
* 
* Note: This is low-level and methods assume, but do not check, all input data is consistent (e.g. that all overlays, 
        the triangle count and enumerator are all consistent with the provided mesh. )
*/
struct FMeshRenderBufferSetConversionUtil
{

	/**
	* Constant color assigned to vertices if no other vertex color is specified
	*/
	FColor ConstantVertexColor = FColor::White;

	/**
	* If true, vertex colors on the FDynamicMesh will be ignored
	*/
	bool bIgnoreVertexColors = false;

	/**
	* If true, a per-triangle color is used to set vertex colors
	*/
	bool bUsePerTriangleColor = false;

	/**
	 * Per-triangle color function. Only called if bUsePerTriangleColor=true
	 */
	TFunction<FColor(const FDynamicMesh3*, int)> PerTriangleColorFunc = nullptr;

	/**
	 * If true, VertexColorRemappingFunc is called on Vertex Colors provided from Mesh to remap them to a different color
	 */
	bool bApplyVertexColorRemapping = false;

	/**
	 * Vertex color remapping function. Only called if bApplyVertexColorRemapping == true, for mesh vertex colors
	 */
	TUniqueFunction<void(FVector4f&)> VertexColorRemappingFunc = nullptr;

	/**
	 * Color Space Transform/Conversion applied to Vertex Colors provided from Mesh Color Overlay Attribute
	 * Color Space Conversion is applied after any Vertex Color Remapping.
	 */
	EDynamicMeshVertexColorTransformMode ColorSpaceTransformMode = EDynamicMeshVertexColorTransformMode::NoTransform;

	/**
	* If true, a facet normals are used instead of mesh normals
	*/
	bool bUsePerTriangleNormals = false;

	/**
	 * If true, populate secondary buffers using SecondaryTriFilterFunc
	 */
	bool bUseSecondaryTriBuffers = false;

	/**
	 * Filter predicate for secondary triangle index buffer. Only called if bUseSecondaryTriBuffers=true
	 */
	TUniqueFunction<bool(const FDynamicMesh3*, int32)> SecondaryTriFilterFunc = nullptr;


	/**
	 * Initialize rendering buffers from given attribute overlays.
	 * Creates three vertices per triangle, IE no shared vertices in buffers.
	 */
	template<typename TriangleEnumerable>
	void InitializeBuffersFromOverlays(
										FMeshRenderBufferSet* RenderBuffers,
										const FDynamicMesh3* Mesh,
										int NumTriangles, const TriangleEnumerable& Enumerable,
										const FDynamicMeshUVOverlay* UVOverlay,
										const FDynamicMeshNormalOverlay* NormalOverlay,
										const FDynamicMeshColorOverlay* ColorOverlay,
										TFunctionRef<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc,
										bool bTrackTriangles = false,
										bool bParallel = false)
	{
		TArray<const FDynamicMeshUVOverlay*> UVOverlays;
		UVOverlays.Add(UVOverlay);
		InitializeBuffersFromOverlays(RenderBuffers, Mesh, NumTriangles, Enumerable,
			UVOverlays, NormalOverlay, ColorOverlay, TangentsFunc, bTrackTriangles, bParallel);
	}


	/**
	* Initialize rendering buffers from given attribute overlays.
	* Creates three vertices per triangle, IE no shared vertices in buffers.
	*/
	template<typename TriangleEnumerable, typename UVOverlayListAllocator>
	void InitializeBuffersFromOverlays( FMeshRenderBufferSet* RenderBuffers,
										const FDynamicMesh3* Mesh,
										int NumTriangles, 
										const TriangleEnumerable& Enumerable,
										const TArray<const FDynamicMeshUVOverlay*, UVOverlayListAllocator>& UVOverlays,
										const FDynamicMeshNormalOverlay* NormalOverlay,
										const FDynamicMeshColorOverlay* ColorOverlay,
										TFunctionRef<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc,
										bool bTrackTriangles = false,
										bool bParallel = false);
												


	/**
	 * Filter the triangles in a FMeshRenderBufferSet into the SecondaryIndexBuffer.
	 * Requires that RenderBuffers->Triangles has been initialized.
	 * @param bDuplicate if set, then primary IndexBuffer is unmodified and SecondaryIndexBuffer contains duplicates. Otherwise triangles are sorted via predicate into either primary or secondary.
	 */
	void GEOMETRYFRAMEWORK_API UpdateSecondaryTriangleBuffer( FMeshRenderBufferSet* RenderBuffers,
										                      const FDynamicMesh3* Mesh,
										                      bool bDuplicate);


	/**
	 * RecomputeRenderBufferTriangleIndexSets re-sorts the existing set of triangles in a FMeshRenderBufferSet
	 * into primary and secondary index buffers. Note that UploadIndexBufferUpdate() must be called
	 * after this function!
	 */
	void GEOMETRYFRAMEWORK_API RecomputeRenderBufferTriangleIndexSets( FMeshRenderBufferSet* RenderBuffers,
												                       const FDynamicMesh3* Mesh);


	/**
	 * Update vertex positions/normals/colors of an existing set of render buffers.
	 * Assumes that buffers were created with unshared vertices, ie three vertices per triangle, eg by InitializeBuffersFromOverlays()
	 */
	template<typename TriangleEnumerable>
	void UpdateVertexBuffersFromOverlays( FMeshRenderBufferSet* RenderBuffers,
										  const FDynamicMesh3* Mesh,
										  int NumTriangles, const TriangleEnumerable& Enumerable,
										  const FDynamicMeshNormalOverlay* NormalOverlay,
										  const FDynamicMeshColorOverlay* ColorOverlay,
										  TFunctionRef<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc,
										  bool bUpdatePositions = true,
										  bool bUpdateNormals = false,
										  bool bUpdateColors = false);


	/**
	 * Update vertex uvs of an existing set of render buffers.
	 * Assumes that buffers were created with unshared vertices, ie three vertices per triangle, eg by InitializeBuffersFromOverlays()
	 */
	template<typename TriangleEnumerable, typename UVOverlayListAllocator>
	void UpdateVertexUVBufferFromOverlays( FMeshRenderBufferSet* RenderBuffers,
										   const FDynamicMesh3* Mesh,
										   int32 NumTriangles, const TriangleEnumerable& Enumerable,
										   const TArray<const FDynamicMeshUVOverlay*, UVOverlayListAllocator>& UVOverlays);


	/**
	* Get the overlay color the FColor,  respecting the ColorSpaceTransformMode utilizing the VertexColorRemappingFunc if requested. 
	*/ 
	FColor GetOverlayColorAsFColor( const FDynamicMeshColorOverlay* ColorOverlay,
		                                                  int32 ElementID)
	{
		checkSlow(ColorOverlay);
		FVector4f UseColor = ColorOverlay->GetElement(ElementID);

		if (bApplyVertexColorRemapping)
		{
			VertexColorRemappingFunc(UseColor);
		}

		if (ColorSpaceTransformMode == EDynamicMeshVertexColorTransformMode::SRGBToLinear)
		{
			// is there a better way to do this? 
			FColor QuantizedSRGBColor = ((FLinearColor)UseColor).ToFColor(false);
			return FLinearColor(QuantizedSRGBColor).ToFColor(false);
		}
		else
		{
			bool bConvertToSRGB = (ColorSpaceTransformMode == EDynamicMeshVertexColorTransformMode::LinearToSRGB);
			return ((FLinearColor)UseColor).ToFColor(bConvertToSRGB);
		}
	}


private:
	using FIndex2i = UE::Geometry::FIndex2i;
	using FIndex3i = UE::Geometry::FIndex3i;
	
};


// Simple tool to initialize a single set of mesh buffers for the entire mesh.  
// for examples of more complicated conversions, see DynamicMeshSceneProxy 
class FDynamicMeshComponentToMeshRenderBufferSet
{

public:
	FMeshRenderBufferSetConversionUtil MeshRenderBufferSetConverter;

	// note, this conversion may recompute the tangents on the DynamicMeshComponent 
	// since a dynamic mesh component with "autocalculated" tangents will compute them on first request
	// @param DynamicMeshComponent - the mesh to be converted
	// @param MeshRenderBufferSet  - the buffer set to be initialized
	// @param bUseComponentSettings - if true, the Component settings will override (and update)  MeshRederBufferSetBuilder.{ColorspaceTransformMode, bUsePerTriangleNormals}.
	GEOMETRYFRAMEWORK_API void Convert(class UDynamicMeshComponent& DynamicMeshComponent, FMeshRenderBufferSet& MeshRenderBufferSet, bool bUseComponentSettings = true);

protected:
	TUniqueFunction<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> MakeTangentsFunc(class UDynamicMeshComponent& DynamicMeshCompnent, bool bSkipAutoCompute = false);
};








// -- template implementations -- // 


template<typename TriangleEnumerable, typename UVOverlayListAllocator>
void FMeshRenderBufferSetConversionUtil::InitializeBuffersFromOverlays( FMeshRenderBufferSet* RenderBuffers,
																	    const FDynamicMesh3* Mesh,
																		int NumTriangles,
																		const TriangleEnumerable& Enumerable,
																	    const TArray<const FDynamicMeshUVOverlay*, UVOverlayListAllocator>& UVOverlays,
																		const FDynamicMeshNormalOverlay* NormalOverlay,
																		const FDynamicMeshColorOverlay* ColorOverlay,
																		TFunctionRef<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc,
																		bool bTrackTriangles,
																		bool bParallel)

{
	RenderBuffers->TriangleCount = NumTriangles;
	if (NumTriangles == 0)
	{
		return;
	}

	bool bHaveColors = (ColorOverlay != nullptr) && (bIgnoreVertexColors == false);

	const int NumVertices = NumTriangles * 3;
	const int NumUVOverlays = UVOverlays.Num();
	const int NumTexCoords = FMath::Max(1, NumUVOverlays);		// must have at least one tex coord

	{
		RenderBuffers->PositionVertexBuffer.Init(NumVertices);
		RenderBuffers->StaticMeshVertexBuffer.Init(NumVertices, NumTexCoords);
		RenderBuffers->ColorVertexBuffer.Init(NumVertices);
		RenderBuffers->IndexBuffer.Indices.AddUninitialized(NumTriangles * 3);
	}

	// build triangle list if requested, or if we are using secondary buffers in which case we need it to filter later
	const bool bBuildTriangleList = bTrackTriangles || bUseSecondaryTriBuffers;

	// populate the triangle array.  we use this for parallelism. 
	TArray<int32> TriangleArray;
	{
		TriangleArray.Reserve(NumTriangles);
		for (int TriangleID : Enumerable)
		{
			TriangleArray.Add(TriangleID);
		}
	}

	TArray<uint32>& IndexBuffer = RenderBuffers->IndexBuffer.Indices;
	{
		// populate the index buffer.
		for (int idx = 0, IdxMax = 3 * NumTriangles; idx < IdxMax; ++idx)
		{
			IndexBuffer[idx] = idx;
		}
	}


	ParallelFor(TriangleArray.Num(), [&](int idx)
	{
		int32 TriangleID = TriangleArray[idx];
		FIndex3i Tri = Mesh->GetTriangle(TriangleID);

	
		FIndex3i TriNormal = (NormalOverlay != nullptr) ? NormalOverlay->GetTriangle(TriangleID) : FIndex3i::Zero();
		FIndex3i TriColor = (ColorOverlay != nullptr) ? ColorOverlay->GetTriangle(TriangleID) : FIndex3i::Zero();

		FColor UniformTriColor = ConstantVertexColor;
		if (bUsePerTriangleColor && PerTriangleColorFunc != nullptr)
		{
			UniformTriColor = PerTriangleColorFunc(Mesh, TriangleID);
			bHaveColors = false;
		}

		int32 VertIdx = idx * 3;
		for (int j = 0; j < 3; ++j)
		{
			RenderBuffers->PositionVertexBuffer.VertexPosition(VertIdx) = (FVector3f)Mesh->GetVertex(Tri[j]);

			FVector3f Normal;
			if (bUsePerTriangleNormals)
			{
				Normal = (FVector3f)Mesh->GetTriNormal(TriangleID);
			}
			else
			{
				Normal = (NormalOverlay != nullptr && TriNormal[j] != FDynamicMesh3::InvalidID) ?
					NormalOverlay->GetElement(TriNormal[j]) : Mesh->GetVertexNormal(Tri[j]);
			}

			// get tangents
			FVector3f TangentX, TangentY;
			TangentsFunc(Tri[j], TriangleID, j, Normal, TangentX, TangentY);

			RenderBuffers->StaticMeshVertexBuffer.SetVertexTangents(VertIdx, TangentX, TangentY, Normal);

			FColor VertexFColor = (bHaveColors && TriColor[j] != FDynamicMesh3::InvalidID) ?
				GetOverlayColorAsFColor(ColorOverlay, TriColor[j]) : UniformTriColor;

			RenderBuffers->ColorVertexBuffer.VertexColor(VertIdx) = VertexFColor;

			VertIdx++;
		}

		for (int32 k = 0; k < NumTexCoords; ++k)
		{
			VertIdx = idx * 3;
			FIndex3i UVTriangle = (k < NumUVOverlays && UVOverlays[k] != nullptr) ? UVOverlays[k]->GetTriangle(TriangleID) : FIndex3i::Invalid();
			for (int j = 0; j < 3; ++j)
			{
				FVector2f UV = (UVTriangle[j] != FDynamicMesh3::InvalidID) ?
					UVOverlays[k]->GetElement(UVTriangle[j]) : FVector2f::Zero();
				RenderBuffers->StaticMeshVertexBuffer.SetVertexUV(VertIdx, k, UV);
				VertIdx++;
			}
		}
	}, !bParallel);

	if (bBuildTriangleList)
	{
		RenderBuffers->Triangles = MoveTemp(TriangleArray);
	}

	// split triangles into secondary buffer (at bit redundant since we just built IndexBuffer, but we may optionally duplicate triangles in the future)
	if (bUseSecondaryTriBuffers)
	{
		RenderBuffers->bEnableSecondaryIndexBuffer = true;
		UpdateSecondaryTriangleBuffer(RenderBuffers, Mesh, false);
	}
}


/**
* Update vertex positions/normals/colors of an existing set of render buffers.
* Assumes that buffers were created with unshared vertices, ie three vertices per triangle, eg by InitializeBuffersFromOverlays()
*/
template<typename TriangleEnumerable>
void FMeshRenderBufferSetConversionUtil::UpdateVertexBuffersFromOverlays(  FMeshRenderBufferSet* RenderBuffers,
																		   const FDynamicMesh3* Mesh,
																		   int NumTriangles, const TriangleEnumerable& Enumerable,
																		   const FDynamicMeshNormalOverlay* NormalOverlay,
																		   const FDynamicMeshColorOverlay* ColorOverlay,
																		   TFunctionRef<void(int, int, int, const FVector3f&, FVector3f&, FVector3f&)> TangentsFunc,
																		   bool bUpdatePositions,
																		   bool bUpdateNormals,
																		   bool bUpdateColors)
{
	if (RenderBuffers->TriangleCount == 0)
	{
		return;
	}

	bool bHaveColors = (ColorOverlay != nullptr) && (bIgnoreVertexColors == false);

	int NumVertices = NumTriangles * 3;
	if ((bUpdatePositions && ensure(RenderBuffers->PositionVertexBuffer.GetNumVertices() == NumVertices) == false)
		|| (bUpdateNormals && ensure(RenderBuffers->StaticMeshVertexBuffer.GetNumVertices() == NumVertices) == false)
		|| (bUpdateColors && ensure(RenderBuffers->ColorVertexBuffer.GetNumVertices() == NumVertices) == false))
	{
		return;
	}

	int VertIdx = 0;
	FVector3f TangentX, TangentY;
	for (int TriangleID : Enumerable)
	{
		FIndex3i Tri = Mesh->GetTriangle(TriangleID);

		FIndex3i TriNormal = (bUpdateNormals && NormalOverlay != nullptr) ? NormalOverlay->GetTriangle(TriangleID) : FIndex3i::Zero();
		FIndex3i TriColor = (bUpdateColors && ColorOverlay != nullptr) ? ColorOverlay->GetTriangle(TriangleID) : FIndex3i::Zero();

		FColor UniformTriColor = ConstantVertexColor;
		if (bUpdateColors && bUsePerTriangleColor && PerTriangleColorFunc != nullptr)
		{
			UniformTriColor = PerTriangleColorFunc(Mesh, TriangleID);
			bHaveColors = false;
		}

		for (int j = 0; j < 3; ++j)
		{
			if (bUpdatePositions)
			{
				RenderBuffers->PositionVertexBuffer.VertexPosition(VertIdx) = (FVector3f)Mesh->GetVertex(Tri[j]);
			}

			if (bUpdateNormals)
			{
				// get normal and tangent
				FVector3f Normal;
				if (bUsePerTriangleNormals)
				{
					Normal = (FVector3f)Mesh->GetTriNormal(TriangleID);
				}
				else
				{
					Normal = (NormalOverlay != nullptr && TriNormal[j] != FDynamicMesh3::InvalidID) ?
						NormalOverlay->GetElement(TriNormal[j]) : Mesh->GetVertexNormal(Tri[j]);
				}

				TangentsFunc(Tri[j], TriangleID, j, Normal, TangentX, TangentY);

				RenderBuffers->StaticMeshVertexBuffer.SetVertexTangents(VertIdx, (FVector3f)TangentX, (FVector3f)TangentY, (FVector3f)Normal);
			}

			if (bUpdateColors)
			{
				FColor VertexFColor = (bHaveColors && TriColor[j] != FDynamicMesh3::InvalidID) ?
					GetOverlayColorAsFColor(ColorOverlay, TriColor[j]) : UniformTriColor;
				RenderBuffers->ColorVertexBuffer.VertexColor(VertIdx) = VertexFColor;
			}

			VertIdx++;
		}
	}
}

template<typename TriangleEnumerable, typename UVOverlayListAllocator>
void FMeshRenderBufferSetConversionUtil::UpdateVertexUVBufferFromOverlays( FMeshRenderBufferSet* RenderBuffers,
																	       const FDynamicMesh3* Mesh,
																	       int32 NumTriangles, const TriangleEnumerable& Enumerable,
																	       const TArray<const FDynamicMeshUVOverlay*, UVOverlayListAllocator>& UVOverlays)
{
	// We align the update to the way we set UV's in InitializeBuffersFromOverlays.

	if (RenderBuffers->TriangleCount == 0)
	{
		return;
	}
	int NumVertices = NumTriangles * 3;
	if (ensure(RenderBuffers->StaticMeshVertexBuffer.GetNumVertices() == NumVertices) == false)
	{
		return;
	}

	int NumUVOverlays = UVOverlays.Num();
	int NumTexCoords = RenderBuffers->StaticMeshVertexBuffer.GetNumTexCoords();
	if (!ensure(NumUVOverlays <= NumTexCoords))
	{
		return;
	}

	// Temporarily stores the UV element indices for all UV channels of a single triangle
	TArray<FIndex3i, TFixedAllocator<MAX_STATIC_TEXCOORDS>> UVTriangles;
	UVTriangles.SetNum(NumTexCoords);

	int VertIdx = 0;
	for (int TriangleID : Enumerable)
	{
		for (int32 k = 0; k < NumTexCoords; ++k)
		{
			UVTriangles[k] = (k < NumUVOverlays && UVOverlays[k] != nullptr) ? UVOverlays[k]->GetTriangle(TriangleID) : FIndex3i::Invalid();
		}

		for (int j = 0; j < 3; ++j)
		{
			for (int32 k = 0; k < NumTexCoords; ++k)
			{
				FVector2f UV = (UVTriangles[k][j] != FDynamicMesh3::InvalidID) ?
					UVOverlays[k]->GetElement(UVTriangles[k][j]) : FVector2f::Zero();
				RenderBuffers->StaticMeshVertexBuffer.SetVertexUV(VertIdx, k, UV);
			}

			++VertIdx;
		}
	}
}

