// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "GeometryBase.h"
#include "HAL/Platform.h"
#include "Templates/UniquePtr.h"

PREDECLARE_USE_GEOMETRY_CLASS(FDynamicMesh3);
class UMaterialInterface;
struct FComponentMaterialSet;

/**
 * FMeshRenderDecomposition represents a decomposition of a mesh into "chunks" of triangles, with associated materials.
 * This is passed to the rendering components to split a mesh into multiple RenderBuffers, for more efficient updating.
 */
class FMeshRenderDecomposition
{
public:

	// Movable
	FMeshRenderDecomposition(FMeshRenderDecomposition&&) = default;
	FMeshRenderDecomposition& operator=(FMeshRenderDecomposition&&) = default;
	// TArray<TUniquePtr> member cannot be default-constructed (in this case we just make it NonCopyable)
	FMeshRenderDecomposition() = default;
	FMeshRenderDecomposition(const FMeshRenderDecomposition&) = delete;
	FMeshRenderDecomposition& operator=(const FMeshRenderDecomposition&) = delete;

	struct FGroup
	{
		TArray<int32> Triangles;
		UMaterialInterface* Material;
	};
	TArray<TUniquePtr<FGroup>> Groups;


	/** Mapping from TriangleID to Groups array index. Initialized by BuildAssociations() */
	TArray<int32> TriangleToGroupMap;

	void Initialize(int32 Count)
	{
		Groups.SetNum(Count);
		for (int32 k = 0; k < Count; ++k)
		{
			Groups[k] = MakeUnique<FGroup>();
		}
	}

	int32 AppendGroup()
	{
		int32 N = Groups.Num();
		Groups.SetNum(N + 1);
		Groups[N] = MakeUnique<FGroup>();
		return N;
	}

	int32 Num() const
	{
		return Groups.Num();
	}

	bool IsGroup(int32 Index) const
	{
		return Groups[Index] != nullptr;
	}

	FGroup& GetGroup(int32 Index)
	{
		return *Groups[Index];
	}

	const FGroup& GetGroup(int32 Index) const
	{
		return *Groups[Index];
	}

	int32 GetGroupForTriangle(int32 TriangleID) const
	{
		return TriangleToGroupMap[TriangleID];
	}

	/**
	 * Construct mappings between mesh and groups (eg TriangleToGroupMap)
	 */
	GEOMETRYFRAMEWORK_API void BuildAssociations(const FDynamicMesh3* Mesh);


	/**
	 * Build decomposition with one group for each MaterialID of mesh
	 */
	static GEOMETRYFRAMEWORK_API void BuildMaterialDecomposition(const FDynamicMesh3* Mesh, const FComponentMaterialSet* MaterialSet, FMeshRenderDecomposition& Decomp);

	/**
	 * Build per-material decomposition, and then split each of those into chunks of at most MaxChunkSize
	 * (actual chunk sizes will be highly variable and some may be very small...)
	 */
	static GEOMETRYFRAMEWORK_API void BuildChunkedDecomposition(const FDynamicMesh3* Mesh, const FComponentMaterialSet* MaterialSet, FMeshRenderDecomposition& Decomp, int32 MaxChunkSize = 1 << 14 /* 16k */ );
};
