// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UDynamicMesh.h"
#include "Components/DynamicMeshComponent.h"

#include "DynamicMeshActor.generated.h"


/**
 * ADynamicMeshActor is an Actor that has a USimpleDynamicMeshComponent as it's RootObject.
 */
UCLASS(ConversionRoot, ComponentWrapperClass, ClassGroup=DynamicMesh, meta = (ChildCanTick), MinimalAPI)
class ADynamicMeshActor : public AActor
{
	GENERATED_UCLASS_BODY()

protected:
	UPROPERTY(Category = DynamicMeshActor, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Mesh,Rendering,Physics,Components|StaticMesh", AllowPrivateAccess = "true"))
	TObjectPtr<class UDynamicMeshComponent> DynamicMeshComponent;

public:
	UFUNCTION(BlueprintCallable, Category = DynamicMeshActor)
	UDynamicMeshComponent* GetDynamicMeshComponent() const { return DynamicMeshComponent; }



	//
	// Mesh Pool support. Meshes can be locally allocated from the Mesh Pool
	// in Blueprints, and then released back to the Pool and re-used. This
	// avoids generating temporary UDynamicMesh instances that need to be
	// garbage-collected. See UDynamicMeshPool for more details.
	//

public:
	/** Control whether the DynamicMeshPool will be created when requested via GetComputeMeshPool() */
	UPROPERTY(Category = "DynamicMeshActor|Advanced", EditAnywhere, BlueprintReadWrite)
	bool bEnableComputeMeshPool = true;
protected:
	/** The internal Mesh Pool, for use in DynamicMeshActor BPs. Use GetComputeMeshPool() to access this, as it will only be created on-demand if bEnableComputeMeshPool = true */
	UPROPERTY(Transient)
	TObjectPtr<UDynamicMeshPool> DynamicMeshPool;

public:
	/** Access the compute mesh pool */
	UFUNCTION(BlueprintCallable, Category = DynamicMeshActor)
	GEOMETRYFRAMEWORK_API UDynamicMeshPool* GetComputeMeshPool();

	/** Request a compute mesh from the Pool, which will return a previously-allocated mesh or add and return a new one. If the Pool is disabled, a new UDynamicMesh will be allocated and returned. */
	UFUNCTION(BlueprintCallable, Category = DynamicMeshActor)
	GEOMETRYFRAMEWORK_API UDynamicMesh* AllocateComputeMesh();

	/** Release a compute mesh back to the Pool */
	UFUNCTION(BlueprintCallable, Category = DynamicMeshActor)
	GEOMETRYFRAMEWORK_API bool ReleaseComputeMesh(UDynamicMesh* Mesh);

	/** Release all compute meshes that the Pool has allocated */
	UFUNCTION(BlueprintCallable, Category = DynamicMeshActor)
	GEOMETRYFRAMEWORK_API void ReleaseAllComputeMeshes();

	/** Release all compute meshes that the Pool has allocated, and then release them from the Pool, so that they will be garbage-collected */
	UFUNCTION(BlueprintCallable, Category = DynamicMeshActor)
	GEOMETRYFRAMEWORK_API void FreeAllComputeMeshes();
};


