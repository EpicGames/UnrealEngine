// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UDynamicMesh.h"

#include "DynamicMeshProcessor.generated.h"

#define UE_API GEOMETRYFRAMEWORK_API



// Blueprints with this parent class can define general processing to apply to a dynamic mesh
// which can then be used to define procedural operations e.g. in Dataflow or other contexts
UCLASS(MinimalAPI, Abstract, BlueprintType, Blueprintable)
class UDynamicMeshProcessorBlueprint : public UObject
{
	GENERATED_BODY()
public:

	/// Apply some processing to change the input mesh
	/// @param TargetMesh Mesh to update (in place)
	/// @param bFailed Set to true to report that the processing has failed
	UFUNCTION(BlueprintImplementableEvent, BlueprintCallable, Category = "MeshProcessor")
	UE_API void ProcessDynamicMesh(UDynamicMesh* TargetMesh, bool& bFailed);

	/** Whether the blueprint must be run on the game thread -- i.e. if it may use nodes that aren't thread safe. */
	UPROPERTY(EditDefaultsOnly, Category = "MeshProcessor")
	bool bRequiresGameThread = true;
};

#undef UE_API
