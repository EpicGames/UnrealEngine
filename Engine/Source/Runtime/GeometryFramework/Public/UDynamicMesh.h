// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DynamicMesh/DynamicMesh3.h"

#include "Changes/MeshVertexChange.h"
#include "Changes/MeshChange.h"
#include "Changes/MeshReplacementChange.h"

#include "UDynamicMesh.generated.h"



/**
 * UDynamicMeshGenerator is an abstract base class for an implementation that can
 * mutate an input mesh into an output mesh. A subclass of this class can be attached to
 * a UDynamicMesh to allow for arbitrarily-complex procedural generation
 */
UCLASS(Abstract, MinimalAPI)
class UDynamicMeshGenerator : public UObject
{
	GENERATED_BODY()
public:
	/** Apply a change to MeshInOut */
	virtual void Generate(FDynamicMesh3& MeshInOut) { };
};





/**
 * EDynamicMeshChangeType is used by FDynamicMeshChangeInfo to indicate a "type" of mesh change
 */
UENUM(BlueprintType)
enum class EDynamicMeshChangeType : uint8
{
	GeneralEdit = 0,
	MeshChange = 1,
	MeshReplacementChange = 2,
	MeshVertexChange = 3,
	
	DeformationEdit = 4,
	AttributeEdit = 5
};

UENUM(BlueprintType)
enum class EDynamicMeshAttributeChangeFlags : uint8
{
	Unknown = 0,
	MeshTopology = 1 << 0,

	VertexPositions = 1 << 1,
	NormalsTangents = 1 << 2,
	VertexColors = 1 << 3,
	UVs = 1 << 4,
	TriangleGroups = 1 << 5
};
ENUM_CLASS_FLAGS(EDynamicMeshAttributeChangeFlags)

/**
 * FDynamicMeshChangeInfo stores information about a change to a UDynamicMesh.
 * This struct is emitted by the UDynamicMesh OnPreMeshChanged() and OnMeshChanged() delegates.
 */
USTRUCT(BlueprintType)
struct FDynamicMeshChangeInfo
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "DynamicMeshChangeInfo")
	EDynamicMeshChangeType Type = EDynamicMeshChangeType::GeneralEdit;

	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "DynamicMeshChangeInfo")
	EDynamicMeshAttributeChangeFlags Flags = EDynamicMeshAttributeChangeFlags::Unknown;

	// for changes that are an FChange, indicates whether this is an 'Apply' or 'Revert' of the FChange
	UPROPERTY(BlueprintReadOnly, EditAnywhere, Category = "DynamicMeshChangeInfo")
	bool bIsRevertChange = false;

	// return the single valid FChange, or null
	const FMeshRegionChangeBase* GetChange() const
	{
		if (MeshChange)
		{
			ensure(!ReplaceChange && !VertexChange); // not valid to have multiple non-null FChanges
			return MeshChange;
		}
		else if (ReplaceChange)
		{
			ensure(!VertexChange); // not valid to have multiple non-null FChanges
			return ReplaceChange;
		}
		else
		{
			return VertexChange;
		}
	}

	//
	// internals
	//
	const FMeshChange* MeshChange = nullptr;
	const FMeshReplacementChange* ReplaceChange = nullptr;
	const FMeshVertexChange* VertexChange = nullptr;
};

// These delegates are used by UDynamicMesh
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDynamicMeshChanged, UDynamicMesh*, FDynamicMeshChangeInfo);
DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FOnDynamicMeshModifiedBP, UDynamicMesh*, Mesh);



/**
 * UDynamicMesh is a UObject container for a FDynamicMesh3. 
 */
UCLASS(BlueprintType, MinimalAPI)
class UDynamicMesh : public UObject,
	public IMeshVertexCommandChangeTarget, 
	public IMeshCommandChangeTarget, 
	public IMeshReplacementCommandChangeTarget
{
	GENERATED_UCLASS_BODY()
public:

	/**
	 * Clear the internal mesh to an empty mesh.
	 * This *does not* allocate a new mesh, so any existing mesh pointers/refs are still valid
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh", meta = (Keywords = "Clear Empty Delete"))
	GEOMETRYFRAMEWORK_API UPARAM(DisplayName = "Target") UDynamicMesh* Reset();

	/**
	 * Clear the internal mesh to a 100x100x100 cube with base at the origin.
	 * This this instead of Reset() if an initially-empty mesh is undesirable (eg for a Component)
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh")
	GEOMETRYFRAMEWORK_API UPARAM(DisplayName = "Target") UDynamicMesh* ResetToCube();

	//
	// Native access/modification functions
	//

	/**
	 * Reset the internal mesh data and then optionally run the MeshGenerator
	 */
	GEOMETRYFRAMEWORK_API virtual void InitializeMesh();

	/**
	 * @return true if the mesh has no triangles
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh")
	GEOMETRYFRAMEWORK_API bool IsEmpty() const;

	/**
	 * @return number of triangles in the mesh
	 */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh")
	GEOMETRYFRAMEWORK_API UPARAM(DisplayName = "Triangle Count") int32 GetTriangleCount() const;


	/** 
	 * @return Const reference to the internal FDynamicMesh3.
	 * @warning Calling ProcessMesh() is preferred! This interface may be  removed in the future
	 */
	const UE::Geometry::FDynamicMesh3& GetMeshRef() const { return *Mesh; }
	/**
	 * @return Const pointer to the internal FDynamicMesh3.
	 * @warning Calling ProcessMesh() is preferred! This interface may be  removed in the future
	 */
	const UE::Geometry::FDynamicMesh3* GetMeshPtr() const { return Mesh.Get(); }

	/**
	 * @return Writable reference to the internal FDynamicMesh3. 
	 * @warning Calling EditMesh() is preferred! This interface may be  removed in the future
	 */
	UE::Geometry::FDynamicMesh3& GetMeshRef() { return *Mesh; }
	/**
	 * @return Writable pointer to the internal FDynamicMesh3.
	 * @warning Calling EditMesh() is preferred! This interface may be  removed in the future
	 */
	UE::Geometry::FDynamicMesh3* GetMeshPtr() { return Mesh.Get(); }


	/** Replace the internal mesh with a copy of MoveMesh */
	GEOMETRYFRAMEWORK_API void SetMesh(const UE::Geometry::FDynamicMesh3& MoveMesh);

	/** Replace the internal mesh with the data in MoveMesh */
	GEOMETRYFRAMEWORK_API void SetMesh(UE::Geometry::FDynamicMesh3&& MoveMesh);

	/**
	 * Apply ProcessFunc to the internal Mesh
	 */
	GEOMETRYFRAMEWORK_API void ProcessMesh(TFunctionRef<void(const UE::Geometry::FDynamicMesh3&)> ProcessFunc) const;

	/**
	 * Apply EditFunc to the internal mesh.
	 * This will broadcast PreMeshChangedEvent, then call EditFunc(), then broadcast MeshChangedEvent and MeshModifiedBPEvent
	 */
	GEOMETRYFRAMEWORK_API void EditMesh(TFunctionRef<void(UE::Geometry::FDynamicMesh3&)> EditFunc,
				  EDynamicMeshChangeType ChangeType = EDynamicMeshChangeType::GeneralEdit,
				  EDynamicMeshAttributeChangeFlags ChangeFlags = EDynamicMeshAttributeChangeFlags::Unknown,
				  bool bDeferChangeEvents = false);

	/**
	 * Take ownership of the internal Mesh, and have it replaced with a new mesh
	 */
	GEOMETRYFRAMEWORK_API TUniquePtr<UE::Geometry::FDynamicMesh3> ExtractMesh();


	//
	// Change support
	//


	// IMeshVertexCommandChangeTarget implementation, allows a FVertexChange to be applied to the mesh
	GEOMETRYFRAMEWORK_API void ApplyChange(const FMeshVertexChange* Change, bool bRevert);

	// IMeshCommandChangeTarget implementation, allows a FMeshChange to be applied to the mesh
	GEOMETRYFRAMEWORK_API void ApplyChange(const FMeshChange* Change, bool bRevert);

	// IMeshReplacementCommandChangeTarget implementation, allows a FMeshReplacementChange to be applied to the mesh
	GEOMETRYFRAMEWORK_API void ApplyChange(const FMeshReplacementChange* Change, bool bRevert);


	//
	// Event support
	//
protected:
	/** Broadcast before the internal mesh is modified by Reset funcs, SetMesh(), EditMesh(), and ApplyChange()'s above */
	FOnDynamicMeshChanged PreMeshChangedEvent;
	/** Broadcast after the internal mesh is modified, in the same cases as PreMeshChangedEvent */
	FOnDynamicMeshChanged MeshChangedEvent;

public:
	/** Broadcast before the internal mesh is modified by Reset funcs, SetMesh(), EditMesh(), and ApplyChange()'s above */
	FOnDynamicMeshChanged& OnPreMeshChanged() { return PreMeshChangedEvent; }
	/** Broadcast after the internal mesh is modified, in the same cases as OnPreMeshChanged */
	FOnDynamicMeshChanged& OnMeshChanged() { return MeshChangedEvent; }

public:
	/** Blueprintable event called when mesh is modified, in the same cases as OnMeshChanged */
	UPROPERTY(BlueprintAssignable, meta = (DisplayName = "MeshModified"))
	FOnDynamicMeshModifiedBP MeshModifiedBPEvent;


	//
	// Realtime Update support.
	// This is intended to be used in situations where the internal mesh is being
	// modified directly, ie instead of via EditMesh(), and we would like to
	// notify listeners of these changes. Generally EditMesh() is preferred but in certain
	// cases (eg like 3D sculpting) it is too complex to refactor all the mesh updates
	// into EditMesh() calls (eg some are done async/etc). So, code that does those
	// kinds of modifications can call PostRealtimeUpdate() to let any interested
	// parties know that the mesh is actively changing.
	//
public:
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnMeshRealtimeUpdate, UDynamicMesh*);

	/**
	 * Multicast delegate that is broadcast whenever PostRealtimeUpdate() is called
	 */
	FOnMeshRealtimeUpdate& OnMeshRealtimeUpdate() { return MeshRealtimeUpdateEvent; }

	/**
	 * Broadcasts FOnMeshRealtimeUpdate
	 */
	GEOMETRYFRAMEWORK_API virtual void PostRealtimeUpdate();

protected:
	FOnMeshRealtimeUpdate MeshRealtimeUpdateEvent;



protected:
	/**
	 * Mesh data, owned by this object.
	 * By default will have Attributes enabled and MaterialID enabled.
	 */
	TUniquePtr<UE::Geometry::FDynamicMesh3> Mesh;

	/**
	 * Allocate a new Mesh (ie pointer will change) and then call InitializeMesh()
	 */
	GEOMETRYFRAMEWORK_API void InitializeNewMesh();

	/**
	 * Internal function that edits the Mesh, but broadcasts PreMeshChangedEvent and MeshChangedEvent, and then MeshModifiedBPEvent
	 */
	GEOMETRYFRAMEWORK_API void EditMeshInternal(TFunctionRef<void(UE::Geometry::FDynamicMesh3&)> EditFunc, const FDynamicMeshChangeInfo& ChangeInfo, bool bDeferChangeEvents = false);


public:
	// serialize Mesh to an Archive
	GEOMETRYFRAMEWORK_API virtual void Serialize(FArchive& Archive) override;

	// serialize Mesh to/from T3D
	GEOMETRYFRAMEWORK_API virtual void ExportCustomProperties(FOutputDevice& Out, uint32 Indent) override;
	GEOMETRYFRAMEWORK_API virtual void ImportCustomProperties(const TCHAR* SourceText, FFeedbackContext* Warn) override;



	//
	// (Preliminary) Procedural Generator support. If a Generator is configured, then each
	// time this mesh is Reset(), it will call MeshGenerator->Generate(). The idea is that 
	// generator will be set to something that emits a new mesh based on external data,
	// for example a procedural primitive Actor/Component could configure a Generator that
	// generates the primitive mesh based on Actor settings.
	//

protected:

	/**
	 * Active mesh generator. If configured, and bEnableMeshGenerator is true, then MeshGenerator->Generate() 
	 * will be called when this mesh is Reset(). The Regenerate() function below can be used to force regeneration.
	 */
	UPROPERTY(Instanced)
	TObjectPtr<UDynamicMeshGenerator> MeshGenerator;

public:

	/**
	 * Controls whether the active Generator (if configured) will be applied when rebuilding the mesh
	 */
	UPROPERTY(EditAnywhere, Category = "Dynamic Mesh")
	bool bEnableMeshGenerator = false;

	/**
	 * Set the active mesh generator. Clears if nullptr
	 */
	GEOMETRYFRAMEWORK_API virtual void SetMeshGenerator(TObjectPtr<UDynamicMeshGenerator> NewGenerator);

	/**
	 * Set the active mesh generator. Clears if nullptr
	 */
	GEOMETRYFRAMEWORK_API virtual void ClearMeshGenerator();

	/**
	 * Reset() the mesh, which will re-run the active MeshGenerator, if bEnableMeshGenerator
	 */
	GEOMETRYFRAMEWORK_API virtual void Regenerate();

};






/**
 * UDynamicMeshPool manages a Pool of UDynamicMesh objects. This allows
 * the meshes to be re-used instead of being garbage-collected. This
 * is intended to be used by Blueprints that need to do procedural geometry
 * operations that generate temporary meshes, as these will commonly run their
 * construction scripts many times as the user (eg) manipulates parameters,
 * and constantly spawning new UDynamicMesh instances will result in enormous
 * memory usage hanging around until GC runs.
 *
 * Usage is to call RequestMesh() to take ownership of an available UDynamicMesh (which
 * will allocate a new one if the pool is empty) and ReturnMesh() to return it to the pool.
 *
 * ReturnAllMeshes() can be called to return all allocated meshes.
 *
 * In both cases, there is nothing preventing you from still holding on to the mesh.
 * So, be careful.
 *
 * FreeAllMeshes() calls ReturnAllMeshes() and then releases the pool's references to
 * the allocated meshes, so they can be Garbage Collected
 *
 * If you Request() more meshes than you Return(), the Pool will still be holding on to
 * references to those meshes, and they will never be Garbage Collected (ie memory leak).
 * As a failsafe, if the number of allocated meshes exceeds geometry.DynamicMesh.MaxPoolSize,
 * the Pool will release all it's references and run garbage collection on the next call to RequestMesh().
 * (Do not rely on this as a memory management strategy)
 *
 * An alternate strategy that could be employed here is for the Pool to not hold
 * references to meshes it has provided, only those that have been explicitly returned.
 * Then non-returned meshes would simply be garbage-collected, however it allows
 * potentially a large amount of memory to be consumed until that occurs.
 *
 * UDynamicMesh::Reset() is called on the object returned to the Pool, which clears
 * the internal FDynamicMesh3 (which uses normal C++ memory management, so no garbage collection involved)
 * So the Pool does not re-use mesh memory, only the UObject containers.
 */
UCLASS(BlueprintType, Transient, MinimalAPI)
class UDynamicMeshPool : public UObject
{
	GENERATED_BODY()
public:
	/** @return an available UDynamicMesh from the pool (possibly allocating a new mesh) */
	UFUNCTION(BlueprintCallable, Category="Dynamic Mesh")
	GEOMETRYFRAMEWORK_API UDynamicMesh* RequestMesh();

	/** Release a UDynamicMesh returned by RequestMesh() back to the pool */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh")
	GEOMETRYFRAMEWORK_API void ReturnMesh(UDynamicMesh* Mesh);

	/** Release all GeneratedMeshes back to the pool */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh")
	GEOMETRYFRAMEWORK_API void ReturnAllMeshes();

	/** Release all GeneratedMeshes back to the pool and allow them to be garbage collected */
	UFUNCTION(BlueprintCallable, Category = "Dynamic Mesh")
	GEOMETRYFRAMEWORK_API void FreeAllMeshes();

protected:
	/** Meshes in the pool that are available */
	UPROPERTY()
	TArray<TObjectPtr<UDynamicMesh>> CachedMeshes;

	/** All meshes the pool has allocated */
	UPROPERTY()
	TArray<TObjectPtr<UDynamicMesh>> AllCreatedMeshes;

private:
	// Typically the mesh pool is used from the game thread, but 
	// in case it is not, we protect pool access w/ this critical section
	mutable FCriticalSection PoolAccessCs;
};
